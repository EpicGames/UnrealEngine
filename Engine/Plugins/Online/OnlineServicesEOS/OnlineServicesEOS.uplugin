{
	"FileVersion": 3,

	"FriendlyName": "Online Services EOS",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Online Services implementation for EOS Account and Game services.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "OnlineServicesEOS",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux",
				"LinuxArm64",
				"Android"
			]
		}
	],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemEOS",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServices",
			"Enabled": true
		},
		{
			"Name": "OnlineServicesEOSGS",
			"Enabled": true
		},
		{
			"Name": "EOSShared",
			"Enabled": true
		}
	]
}