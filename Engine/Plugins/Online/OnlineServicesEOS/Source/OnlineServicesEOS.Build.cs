// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineServicesEOS : ModuleRules
{
	public OnlineServicesEOS(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"OnlineServicesInterface",
				"OnlineServicesCommon",
				"OnlineServicesEOSGS"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreOnline",
				"CoreUObject",
				"EOSSDK",
				"EOSShared",
				"OnlineServicesEpicCommon"
			}
		);
	}
}
