// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AuthEOS.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "Containers/StaticArray.h"
#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/AuthErrors.h"
#include "Online/EpicAccountIdResolver.h"
#include "Online/EpicProductUserIdResolver.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOS.h"
#include "Online/OnlineServicesEOS.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"

#include "eos_auth.h"
#include "eos_connect.h"
#include "eos_userinfo.h"

namespace UE::Online {

struct FAuthEOSLoginConfig
{
	TArray<FString> DefaultScopes;
	bool bAutoLinkAccount = true;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAuthEOSLoginConfig)
	ONLINE_STRUCT_FIELD(FAuthEOSLoginConfig, DefaultScopes),
	ONLINE_STRUCT_FIELD(FAuthEOSLoginConfig, bAutoLinkAccount)
END_ONLINE_STRUCT_META()

/* Meta*/ }


namespace
{

#define UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME TEXT("AccountInfoEOS")
#define UE_ONLINE_AUTH_EOS_CONTINUANCE_DATA_KEY_NAME TEXT("ContinuanceToken")
#define UE_ONLINE_AUTH_EOS_SELECTED_ACCOUNT_ID_KEY_NAME TEXT("SelectedAccountId")

/* anonymous */ }

namespace ELinkAccountTag
{
const FName InternalAccount = TEXT("InternalAccount");
}

FAuthEOS::FAuthEOS(FOnlineServicesCommon& InServices)
	: Super(InServices)
{
}

void FAuthEOS::Initialize()
{
	Super::Initialize();

	UserInfoHandle = EOS_Platform_GetUserInfoInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(UserInfoHandle != nullptr);
}

TOnlineAsyncOpHandle<FAuthLogin> FAuthEOS::Login(FAuthLogin::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLogin> Op = GetOp<FAuthLogin>(MoveTemp(Params));
	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const FAuthLogin::Params& Params = InAsyncOp.GetParams();

		// Check that user is valid.
		if (!Params.PlatformUserId.IsValid())
		{
			InAsyncOp.SetError(Errors::InvalidParams());
			return;
		}

		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.PlatformUserId);
		if (AccountInfoEOS)
		{
			InAsyncOp.SetError(Errors::Auth::AlreadyLoggedIn());
			return;
		}

		AccountInfoEOS = MakeShared<FAccountInfoEOS>();
		AccountInfoEOS->PlatformUserId = Params.PlatformUserId;
		AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;

		// New login attempt - Clear the continuance token for the last login attempt for the user.
		if (FUserScopedData* UserData = GetUserScopedData(Params.PlatformUserId))
		{
			UserData->LastLoginContinuationId = FLoginContinuationId();
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME, AccountInfoEOS.ToSharedRef());
	})
	// Step 2: Login EAS.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		const FAuthLogin::Params& Params = InAsyncOp.GetParams();

		FAuthEOSLoginConfig AuthEOSLoginConfig;
		LoadConfig(AuthEOSLoginConfig, TEXT("Login"));

		FAuthLoginEASImpl::Params LoginParams;
		LoginParams.PlatformUserId = Params.PlatformUserId;
		LoginParams.CredentialsType = Params.CredentialsType;
		LoginParams.CredentialsId = Params.CredentialsId;
		LoginParams.CredentialsToken = Params.CredentialsToken;
		LoginParams.Scopes = !Params.Scopes.IsEmpty() ? Params.Scopes : AuthEOSLoginConfig.DefaultScopes;
		LoginParams.bAutoLinkAccount = AuthEOSLoginConfig.bAutoLinkAccount;

		LoginEASImpl(LoginParams)
		.Next([this, Promise = MoveTemp(Promise), WeakOp = InAsyncOp.AsWeak()](TDefaultErrorResult<FAuthLoginEASImpl>&& LoginResult) mutable -> void
		{
			if (TSharedPtr<TOnlineAsyncOp<FAuthLogin>> Op = WeakOp.Pin())
			{
				const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(*Op, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

				if (LoginResult.IsError())
				{
					UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::Login] Failure: LoginEASImpl %ls", *LoginResult.GetErrorValue().GetLogString());
					Op->SetError(MoveTemp(LoginResult.GetErrorValue()));
				}
				else
				{
					// Cache EpicAccountId on successful EAS login.
					AccountInfoEOS->EpicAccountId = LoginResult.GetOkValue().EpicAccountId;
				}
			}

			Promise.EmplaceValue();
		});

		return Future;
	})
	// Step 3: Fetch external auth credentials for connect login.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const FAuthLogin::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

		TPromise<FAuthLoginConnectImpl::Params> Promise;
		TFuture<FAuthLoginConnectImpl::Params> Future = Promise.GetFuture();

		TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> AuthTokenResult = GetExternalAuthTokenImpl(FAuthGetExternalAuthTokenImpl::Params{AccountInfoEOS->EpicAccountId});
		if (AuthTokenResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::Login] Failure: GetExternalAuthTokenImpl %ls", *AuthTokenResult.GetErrorValue().GetLogString());

			// Failed to acquire token - logout EAS.
			LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
				.Next([AsyncOp = InAsyncOp.AsShared(), Error = MoveTemp(AuthTokenResult.GetErrorValue()), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
				{
					AsyncOp->SetError(MoveTemp(Error));
					Promise.EmplaceValue(FAuthLoginConnectImpl::Params{});
				});

			return Future;
		}

		Promise.EmplaceValue(FAuthLoginConnectImpl::Params{Params.PlatformUserId, MoveTemp(AuthTokenResult.GetOkValue().Token)});
		return Future;
	})
	// Step 4: Attempt connect login. On connect login failure handle logout of EAS.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp, FAuthLoginConnectImpl::Params&& LoginConnectParams)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		// Attempt connect login.
		LoginConnectImpl(LoginConnectParams)
		.Next([this, AccountInfoEOS, WeakOp = InAsyncOp.AsWeak(), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLoginConnectImpl>&& LoginResult) mutable -> void
		{
			if (TSharedPtr<TOnlineAsyncOp<FAuthLogin>> AsyncOp = WeakOp.Pin())
			{
				if (LoginResult.IsError())
				{
					UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::Login] Failure: LoginConnectImpl %ls", *LoginResult.GetErrorValue().GetLogString());

					LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
					.Next([AsyncOp, Error = MoveTemp(LoginResult.GetErrorValue()), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
					{
						AsyncOp->SetError(MoveTemp(Error));
						Promise.EmplaceValue();
					});
				}
				else
				{
					// Successful login.
					AccountInfoEOS->ProductUserId = LoginResult.GetOkValue().ProductUserId;
					Promise.EmplaceValue();
				}
			}
			else
			{
				Promise.EmplaceValue();
			}
		});

		return Future;
	})
	// Step 5: Fetch dependent data.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

		// Get display name
		EOS_UserInfo_CopyBestDisplayNameOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAME_API_LATEST, 1);
		Options.LocalUserId = AccountInfoEOS->EpicAccountId;
		Options.TargetUserId = AccountInfoEOS->EpicAccountId;

		EOS_UserInfo_BestDisplayName* BestDisplayName;
		EOS_EResult CopyBestDisplayNameResult = EOS_UserInfo_CopyBestDisplayName(UserInfoHandle, &Options, &BestDisplayName);

		if (CopyBestDisplayNameResult == EOS_EResult::EOS_UserInfo_BestDisplayNameIndeterminate)
		{
			EOS_UserInfo_CopyBestDisplayNameWithPlatformOptions WithPlatformOptions = {};
			WithPlatformOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAMEWITHPLATFORM_API_LATEST, 1);
			WithPlatformOptions.LocalUserId = AccountInfoEOS->EpicAccountId;
			WithPlatformOptions.TargetUserId = AccountInfoEOS->EpicAccountId;
			WithPlatformOptions.TargetPlatformType = EOS_OPT_Epic;

			CopyBestDisplayNameResult = EOS_UserInfo_CopyBestDisplayNameWithPlatform(UserInfoHandle, &WithPlatformOptions, &BestDisplayName);
		}

		if (CopyBestDisplayNameResult == EOS_EResult::EOS_Success)
		{
			AccountInfoEOS->Attributes.Emplace(AccountAttributeData::DisplayName, *GetBestDisplayNameStr(*BestDisplayName));
			EOS_UserInfo_BestDisplayName_Release(BestDisplayName);
		}
		else
		{
			FOnlineError CopyUserInfoError(Errors::FromEOSResult(CopyBestDisplayNameResult));
			UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::Login] Failure: EOS_UserInfo_CopyBestDisplayName %ls", *CopyUserInfoError.GetLogString());

			TPromise<void> Promise;
			TFuture<void> Future = Promise.GetFuture();
			LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
			.Next([this, AccountInfoEOS, WeakOp = InAsyncOp.AsWeak(), Error = MoveTemp(CopyUserInfoError), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
			{
				if (TSharedPtr<TOnlineAsyncOp<FAuthLogin>> AsyncOp = WeakOp.Pin())
				{
					LogoutConnectImpl(FAuthLogoutConnectImpl::Params{ AccountInfoEOS->ProductUserId })
					.Next([AsyncOp, Error = MoveTemp(Error), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutConnectImpl>&&) mutable -> void
					{
						AsyncOp->SetError(MoveTemp(Error));
						Promise.EmplaceValue();
					});
				}
				else
				{
					Promise.EmplaceValue();
				}
			});

			return Future;
		}

		return MakeFulfilledPromise<void>().GetFuture();
	})
	// Step 6: bookkeeping and notifications.
	.Then([this](TOnlineAsyncOp<FAuthLogin>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);
		AccountInfoEOS->LoginStatus = ELoginStatus::LoggedIn;
		AccountInfoEOS->AccountId = CreateAccountId(AccountInfoEOS->EpicAccountId, AccountInfoEOS->ProductUserId);
		AccountInfoRegistryEOS.Register(AccountInfoEOS);

		UE_LOGF(LogOnlineServices, Log, "[FAuthEOS::Login] Successfully logged in as [%ls]", *ToLogString(AccountInfoEOS->AccountId));
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });
		InAsyncOp.SetResult(FAuthLogin::Result{AccountInfoEOS});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthLinkAccount> FAuthEOS::LinkAccount(FAuthLinkAccount::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLinkAccount> Op = GetOp<FAuthLinkAccount>(MoveTemp(Params));
	// Step 1: Set up operation data.
	Op->Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp)
	{
		const FAuthLinkAccount::Params& Params = InAsyncOp.GetParams();

		// Check that user is valid.
		if (!Params.PlatformUserId.IsValid())
		{
			InAsyncOp.SetError(Errors::InvalidParams());
			return;
		}

		// Check that user scoped data exists for user.
		FUserScopedData* UserData = FAuthEOSGS::GetUserScopedData(Params.PlatformUserId);
		if (UserData == nullptr)
		{
			InAsyncOp.SetError(Errors::InvalidParams());
			return;
		}

		// Make sure continuation exists.
		const FLoginContinuationData* ContinuanceData = Algo::FindByPredicate(UserData->LoginContinuations, [&Params](const FLoginContinuationData& Continuation)->bool { return Continuation.ContinuationId == Params.ContinuationId; });
		if (ContinuanceData == nullptr)
		{
			InAsyncOp.SetError(Errors::InvalidParams());
			return;
		}

		InAsyncOp.Data.Set<FLoginContinuationData>(UE_ONLINE_AUTH_EOS_CONTINUANCE_DATA_KEY_NAME, *ContinuanceData);
	})
	// Step 2: Call link account.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp, TPromise<const EOS_Auth_LinkAccountCallbackInfo*>&& Promise)
	{
		const FAuthLinkAccount::Params& Params = InAsyncOp.GetParams();
		const FLoginContinuationData& LoginContinuationData = GetOpDataChecked<FLoginContinuationData>(InAsyncOp, UE_ONLINE_AUTH_EOS_CONTINUANCE_DATA_KEY_NAME);
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.PlatformUserId);

		EOS_Auth_LinkAccountOptions LinkAccountOptions = {};
		LinkAccountOptions.ApiVersion = 1;
		LinkAccountOptions.ContinuanceToken = LoginContinuationData.ContinuanceToken;
		LinkAccountOptions.LinkAccountFlags = (!Params.Tags.Contains(ELinkAccountTag::InternalAccount)) ? LoginContinuationData.LinkAccountFlags : EOS_ELinkAccountFlags::EOS_LA_NoFlags;
		LinkAccountOptions.LocalUserId = AccountInfoEOS ? AccountInfoEOS->EpicAccountId : nullptr;
		UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_LINKACCOUNT_API_LATEST, 1);

		EOS_Async(EOS_Auth_LinkAccount, AuthHandle, LinkAccountOptions, MoveTemp(Promise));
	})
	// Step 3: Handle link account result.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp, const EOS_Auth_LinkAccountCallbackInfo* Data)
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FAuthEOS::LinkAccount] EOS_Auth_LinkAccount Result: [%ls]", *LexToString(Data->ResultCode));

		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			InAsyncOp.Data.Set<EOS_EpicAccountId>(UE_ONLINE_AUTH_EOS_SELECTED_ACCOUNT_ID_KEY_NAME, Data->SelectedAccountId);
		}
		else
		{
			InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
		}
	})
	// Step 4: Link account success handling / user info setup.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp)
	{
		const FAuthLinkAccount::Params& Params = InAsyncOp.GetParams();

		// Remove continuance token.
		FUserScopedData* UserData = FAuthEOSGS::GetUserScopedData(Params.PlatformUserId);
		check(UserData);
		UserData->LastLoginContinuationId = FLoginContinuationId();
		UserData->LoginContinuations.SetNum(Algo::RemoveIf(UserData->LoginContinuations, [&Params](const FLoginContinuationData& Continuation)
		{
			return Continuation.ContinuationId == Params.ContinuationId;
		}));

		// Create or fetch FAccountInfoEOS for user.
		const EOS_EpicAccountId SelectedUserAccount = GetOpDataChecked<EOS_EpicAccountId>(InAsyncOp, UE_ONLINE_AUTH_EOS_SELECTED_ACCOUNT_ID_KEY_NAME);
		TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.PlatformUserId);
		const bool bWasLoggedIn = AccountInfoEOS && AccountInfoEOS->EpicAccountId == SelectedUserAccount;
		const bool bChoseDifferentAccount = bWasLoggedIn && AccountInfoEOS->EpicAccountId != SelectedUserAccount;

		// Notify logout of previous account.
		if (AccountInfoEOS && bChoseDifferentAccount)
		{
			AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;
			OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS.ToSharedRef(), AccountInfoEOS->LoginStatus });
			AccountInfoRegistryEOS.Unregister(AccountInfoEOS->AccountId);
			AccountInfoEOS = nullptr;
		}

		// Setup user account if user is not already logged in.
		if (AccountInfoEOS == nullptr)
		{
			AccountInfoEOS = MakeShared<FAccountInfoEOS>();
			AccountInfoEOS->PlatformUserId = Params.PlatformUserId;
			AccountInfoEOS->EpicAccountId = SelectedUserAccount;
			AccountInfoEOS->LoginStatus = ELoginStatus::NotLoggedIn;
		}

		// Set user auth data on operation.
		InAsyncOp.Data.Set<TSharedRef<FAccountInfoEOS>>(UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME, AccountInfoEOS.ToSharedRef());
	})
	// Step 5: Fetch external auth credentials for connect login.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp)
	{
		const FAuthLinkAccount::Params& Params = InAsyncOp.GetParams();
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

		TPromise<FAuthLoginConnectImpl::Params> Promise;
		TFuture<FAuthLoginConnectImpl::Params> Future = Promise.GetFuture();

		if (AccountInfoEOS->LoginStatus == ELoginStatus::NotLoggedIn)
		{
			TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> AuthTokenResult = GetExternalAuthTokenImpl(FAuthGetExternalAuthTokenImpl::Params{AccountInfoEOS->EpicAccountId});
			if (AuthTokenResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::LinkAccount] Failure: GetExternalAuthTokenImpl %ls", *AuthTokenResult.GetErrorValue().GetLogString());

				// Failed to acquire token - logout EAS.
				LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
				.Next([AsyncOp = InAsyncOp.AsShared(), Error = MoveTemp(AuthTokenResult.GetErrorValue()), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
				{
					AsyncOp->SetError(MoveTemp(Error));
					Promise.EmplaceValue(FAuthLoginConnectImpl::Params{});
				});

				return Future;
			}

			Promise.EmplaceValue(FAuthLoginConnectImpl::Params{Params.PlatformUserId, MoveTemp(AuthTokenResult.GetOkValue().Token)});
		}
		else
		{
			// No connect login is needed - user is already logged in.
			Promise.EmplaceValue(FAuthLoginConnectImpl::Params{});
		}
		return Future;
	})
	// Step 6: Attempt connect login. On connect login failure handle logout of EAS.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp, FAuthLoginConnectImpl::Params&& LoginConnectParams)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

		TPromise<void> Promise;
		TFuture<void> Future = Promise.GetFuture();

		if (AccountInfoEOS->LoginStatus == ELoginStatus::NotLoggedIn)
		{
			// Attempt connect login.
			LoginConnectImpl(LoginConnectParams)
			.Next([this, AccountInfoEOS, WeakOp = InAsyncOp.AsWeak(), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLoginConnectImpl>&& LoginResult) mutable -> void
			{
				if (TSharedPtr<TOnlineAsyncOp<FAuthLinkAccount>> AsyncOp = WeakOp.Pin())
				{
					if (LoginResult.IsError())
					{
						UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::LinkAccount] Failure: LoginConnectImpl %ls", *LoginResult.GetErrorValue().GetLogString());

						LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
						.Next([AsyncOp, Error = MoveTemp(LoginResult.GetErrorValue()), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
						{
							AsyncOp->SetError(MoveTemp(Error));
							Promise.EmplaceValue();
						});
					}
					else
					{
						// Successful login.
						AccountInfoEOS->ProductUserId = LoginResult.GetOkValue().ProductUserId;
						Promise.EmplaceValue();
					}
				}
				else
				{
					Promise.EmplaceValue();
				}
			});
		}
		else
		{
			// No connect login is needed - user is already logged in.
			Promise.EmplaceValue();
		}

		return Future;
	})
	// Step 7: Fetch dependent data.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);

		if (AccountInfoEOS->LoginStatus == ELoginStatus::NotLoggedIn)
		{
			// Get display name
			EOS_UserInfo_CopyBestDisplayNameOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAME_API_LATEST, 1);
			Options.LocalUserId = AccountInfoEOS->EpicAccountId;
			Options.TargetUserId = AccountInfoEOS->EpicAccountId;

			EOS_UserInfo_BestDisplayName* BestDisplayName;
			EOS_EResult CopyBestDisplayNameResult = EOS_UserInfo_CopyBestDisplayName(UserInfoHandle, &Options, &BestDisplayName);

			if (CopyBestDisplayNameResult == EOS_EResult::EOS_UserInfo_BestDisplayNameIndeterminate)
			{
				EOS_UserInfo_CopyBestDisplayNameWithPlatformOptions WithPlatformOptions = {};
				WithPlatformOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAMEWITHPLATFORM_API_LATEST, 1);
				WithPlatformOptions.LocalUserId = AccountInfoEOS->EpicAccountId;
				WithPlatformOptions.TargetUserId = AccountInfoEOS->EpicAccountId;
				WithPlatformOptions.TargetPlatformType = EOS_OPT_Epic;

				CopyBestDisplayNameResult = EOS_UserInfo_CopyBestDisplayNameWithPlatform(UserInfoHandle, &WithPlatformOptions, &BestDisplayName);
			}

			if (CopyBestDisplayNameResult == EOS_EResult::EOS_Success)
			{
				AccountInfoEOS->Attributes.Emplace(AccountAttributeData::DisplayName, *GetBestDisplayNameStr(*BestDisplayName));
				EOS_UserInfo_BestDisplayName_Release(BestDisplayName);
			}
			else
			{
				FOnlineError CopyUserInfoError(Errors::FromEOSResult(CopyBestDisplayNameResult));
				UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::LinkAccount] Failure: EOS_UserInfo_CopyBestDisplayName %ls", *CopyUserInfoError.GetLogString());

				TPromise<void> Promise;
				TFuture<void> Future = Promise.GetFuture();
				LogoutEASImpl(FAuthLogoutEASImpl::Params{ AccountInfoEOS->EpicAccountId })
				.Next([this, AccountInfoEOS, WeakOp = InAsyncOp.AsWeak(), Error = MoveTemp(CopyUserInfoError), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutEASImpl>&&) mutable -> void
				{
					if (TSharedPtr<TOnlineAsyncOp<FAuthLinkAccount>> AsyncOp = WeakOp.Pin())
					{
						LogoutConnectImpl(FAuthLogoutConnectImpl::Params{ AccountInfoEOS->ProductUserId })
						.Next([AsyncOp, Error = MoveTemp(Error), Promise = MoveTemp(Promise)](TDefaultErrorResult<FAuthLogoutConnectImpl>&&) mutable -> void
						{
							AsyncOp->SetError(MoveTemp(Error));
							Promise.EmplaceValue();
						});
					}
					else
					{
						Promise.EmplaceValue();
					}
				});

				return Future;
			}
		}

		return MakeFulfilledPromise<void>().GetFuture();
	})
	// Step 8: bookkeeping and notifications.
	.Then([this](TOnlineAsyncOp<FAuthLinkAccount>& InAsyncOp)
	{
		const TSharedRef<FAccountInfoEOS>& AccountInfoEOS = GetOpDataChecked<TSharedRef<FAccountInfoEOS>>(InAsyncOp, UE_ONLINE_AUTH_EOS_ACCOUNT_INFO_KEY_NAME);
		const bool bUserWasLoggedIn = AccountInfoEOS->LoginStatus == ELoginStatus::LoggedIn;

		if (bUserWasLoggedIn)
		{
			UE_LOGF(LogOnlineServices, Log, "[FAuthEOS::LinkAccount] Successfully linked account. AccountId: %ls", *ToLogString(AccountInfoEOS->AccountId));
		}
		else
		{
			AccountInfoEOS->LoginStatus = ELoginStatus::LoggedIn;
			AccountInfoEOS->AccountId = CreateAccountId(AccountInfoEOS->EpicAccountId, AccountInfoEOS->ProductUserId);
			AccountInfoRegistryEOS.Register(AccountInfoEOS);
			UE_LOGF(LogOnlineServices, Log, "[FAuthEOS::LinkAccount] Successfully logged in. AccountId: %ls", *ToLogString(AccountInfoEOS->AccountId));
			OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfoEOS, AccountInfoEOS->LoginStatus });
		}

		InAsyncOp.SetResult(FAuthLinkAccount::Result{AccountInfoEOS});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> FAuthEOS::QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalServerAuthTicket> Op = GetJoinableOp<FAuthQueryExternalServerAuthTicket>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FAuthQueryExternalServerAuthTicket>& InAsyncOp)
		{
			const FAuthQueryExternalServerAuthTicket::Params& Params = InAsyncOp.GetParams();
			TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalAccountId);
			if (!AccountInfoEOS)
			{
				InAsyncOp.SetError(Errors::InvalidParams());
				return;
			}

			EOS_Auth_CopyUserAuthTokenOptions CopyUserAuthTokenOptions = {};
			CopyUserAuthTokenOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST, 1);

			EOS_Auth_Token* AuthToken = nullptr;

			EOS_EResult Result = EOS_Auth_CopyUserAuthToken(AuthHandle, &CopyUserAuthTokenOptions, AccountInfoEOS->EpicAccountId, &AuthToken);
			if (Result == EOS_EResult::EOS_Success)
			{
				ON_SCOPE_EXIT
				{
					EOS_Auth_Token_Release(AuthToken);
				};

				FExternalServerAuthTicket ExternalServerAuthTicket;
				ExternalServerAuthTicket.Type = ExternalLoginType::Epic;
				ExternalServerAuthTicket.Data = UTF8_TO_TCHAR(AuthToken->AccessToken);
				InAsyncOp.SetResult(FAuthQueryExternalServerAuthTicket::Result{ MoveTemp(ExternalServerAuthTicket) });
			}
			else
			{
				InAsyncOp.SetError(Errors::FromEOSResult(Result));
			}
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> FAuthEOS::QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalAuthToken> Op = GetJoinableOp<FAuthQueryExternalAuthToken>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp)
		{
			const FAuthQueryExternalAuthToken::Params& Params = InAsyncOp.GetParams();
			TSharedPtr<FAccountInfoEOS> AccountInfoEOS = AccountInfoRegistryEOS.Find(Params.LocalAccountId);
			if (!AccountInfoEOS)
			{
				InAsyncOp.SetError(Errors::InvalidParams());
				return;
			}

			// The primary external auth method is an id token.
			if (Params.Method == EExternalAuthTokenMethod::Primary)
			{
				TDefaultErrorResult<FAuthGetExternalAuthTokenImpl> AuthTokenResult = GetExternalAuthTokenImpl(FAuthGetExternalAuthTokenImpl::Params{ AccountInfoEOS->EpicAccountId });
				if (AuthTokenResult.IsError())
				{
					UE_LOGF(LogOnlineServices, Warning, "[FAuthEOS::QueryExternalAuthToken] Failure: GetExternalAuthTokenImpl %ls", *AuthTokenResult.GetErrorValue().GetLogString());
					InAsyncOp.SetError(MoveTemp(AuthTokenResult.GetErrorValue()));
					return;
				}

				InAsyncOp.SetResult(FAuthQueryExternalAuthToken::Result{ MoveTemp(AuthTokenResult.GetOkValue().Token) });
			}
			// The secondary external auth method is an EAS refresh token.
			else if (Params.Method == EExternalAuthTokenMethod::Secondary)
			{
				EOS_Auth_CopyUserAuthTokenOptions CopyUserAuthTokenOptions = {};
				CopyUserAuthTokenOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST, 1);

				EOS_Auth_Token* AuthToken = nullptr;

				EOS_EResult Result = EOS_Auth_CopyUserAuthToken(AuthHandle, &CopyUserAuthTokenOptions, AccountInfoEOS->EpicAccountId, &AuthToken);
				if (Result == EOS_EResult::EOS_Success)
				{
					ON_SCOPE_EXIT
					{
						EOS_Auth_Token_Release(AuthToken);
					};

					FExternalAuthToken ExternalAuthToken;
					ExternalAuthToken.Type = ExternalLoginType::Epic;
					ExternalAuthToken.Data = UTF8_TO_TCHAR(AuthToken->RefreshToken);
					InAsyncOp.SetResult(FAuthQueryExternalAuthToken::Result{ MoveTemp(ExternalAuthToken) });
				}
				else
				{
					InAsyncOp.SetError(Errors::FromEOSResult(Result));
				}
			}
			else
			{
				InAsyncOp.SetError(Errors::InvalidParams());
			}
		})
		.Enqueue(GetSerialQueue());
	}

	return Op->GetHandle();
}

TOnlineResult<FAuthGetLinkAccountContinuationId> FAuthEOS::GetLinkAccountContinuationId(FAuthGetLinkAccountContinuationId::Params&& Params) const
{
	if (!Params.PlatformUserId.IsValid())
	{
		return TOnlineResult<FAuthGetLinkAccountContinuationId>(Errors::InvalidUser());
	}

	const FUserScopedData* UserData = GetUserScopedData(Params.PlatformUserId);
	if (UserData == nullptr)
	{
		UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to find user scoped data. PlatformUserId: %ls.", __FUNCTION__, *ToLogString(Params.PlatformUserId));
		return TOnlineResult<FAuthGetLinkAccountContinuationId>(Errors::NotFound());
	}

	if (!UserData->LastLoginContinuationId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to find valid login continuation. PlatformUserId: %ls.", __FUNCTION__, *ToLogString(Params.PlatformUserId));
		return TOnlineResult<FAuthGetLinkAccountContinuationId>(Errors::NotFound());
	}

	UE_LOGF(LogOnlineServices, Verbose, "[%s]: Found continuation id. PlatformUserId: %ls, ContinuationId: %ls.", __FUNCTION__, *ToLogString(Params.PlatformUserId), *ToLogString(UserData->LastLoginContinuationId));
	return TOnlineResult<FAuthGetLinkAccountContinuationId>(FAuthGetLinkAccountContinuationId::Result{UserData->LastLoginContinuationId});
}

using FEpicAccountIdStrBuffer = TStaticArray<char, EOS_EPICACCOUNTID_MAX_LENGTH + 1>;

TFuture<TArray<FAccountId>> FAuthEOS::ResolveAccountIds(const FAccountId& LocalAccountId, const TArray<EOS_EpicAccountId>& InEpicAccountIds)
{
	// Search for all the account id's
	TArray<FAccountId> AccountIdHandles;
	AccountIdHandles.Reserve(InEpicAccountIds.Num());
	TArray<EOS_EpicAccountId> MissingEpicAccountIds;
	MissingEpicAccountIds.Reserve(InEpicAccountIds.Num());
	for (const EOS_EpicAccountId EpicAccountId : InEpicAccountIds)
	{
		if (!EOS_EpicAccountId_IsValid(EpicAccountId))
		{
			return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
		}

		FAccountId Found = FindAccountId(EpicAccountId);
		if (!Found.IsValid())
		{
			MissingEpicAccountIds.Emplace(EpicAccountId);
		}
		AccountIdHandles.Emplace(MoveTemp(Found));
	}
	if (MissingEpicAccountIds.IsEmpty())
	{
		// We have them all, so we can just return
		return MakeFulfilledPromise<TArray<FAccountId>>(MoveTemp(AccountIdHandles)).GetFuture();
	}

	// If we failed to find all the handles, we need to query, which requires a valid LocalAccountId
	// Note this is unavailable on Dedicated Servers as well, unlike EOS_Connect_QueryProductUserIdMappings
	if (!FOnlineAccountIdRegistryEOS::ValidateOnlineId(LocalAccountId))
	{
		checkNoEntry();
		return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
	}

	TPromise<TArray<FAccountId>> Promise;
	TFuture<TArray<FAccountId>> Future = Promise.GetFuture();

	TArray<FEpicAccountIdStrBuffer> EpicAccountIdStrsToQuery;
	EpicAccountIdStrsToQuery.Reserve(MissingEpicAccountIds.Num());
	for (const EOS_EpicAccountId EpicAccountId : MissingEpicAccountIds)
	{
		FEpicAccountIdStrBuffer& EpicAccountIdStr = EpicAccountIdStrsToQuery.Emplace_GetRef();
		int32_t BufferSize = sizeof(EpicAccountIdStr);
		if (!EOS_EpicAccountId_IsValid(EpicAccountId) ||
			EOS_EpicAccountId_ToString(EpicAccountId, EpicAccountIdStr.GetData(), &BufferSize) != EOS_EResult::EOS_Success)
		{
			checkNoEntry();
			return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
		}
	}

	TArray<const char*> EpicAccountIdStrPtrs;
	Algo::Transform(EpicAccountIdStrsToQuery, EpicAccountIdStrPtrs, [](const FEpicAccountIdStrBuffer& Str) { return &Str[0]; });

	EOS_Connect_QueryExternalAccountMappingsOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_QUERYEXTERNALACCOUNTMAPPINGS_API_LATEST, 1);
	Options.LocalUserId = GetProductUserIdChecked(LocalAccountId);
	Options.AccountIdType = EOS_EExternalAccountType::EOS_EAT_EPIC;
	Options.ExternalAccountIds = (const char**)EpicAccountIdStrPtrs.GetData();
	Options.ExternalAccountIdCount = EpicAccountIdStrPtrs.Num();

	TSharedPtr<IEOSFastTickLock> EOSSDKFastTickLock = GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle()->GetFastTickLock();

	EOS_Async(EOS_Connect_QueryExternalAccountMappings, ConnectHandle, Options,
	[this, WeakThis = AsWeak(), InEpicAccountIds, Promise = MoveTemp(Promise), EOSSDKFastTickLock = MoveTemp(EOSSDKFastTickLock)](const EOS_Connect_QueryExternalAccountMappingsCallbackInfo* Data) mutable -> void
	{
		TArray<FAccountId> AccountIds;
		if (const TSharedPtr<IAuth> StrongThis = WeakThis.Pin())
		{
			AccountIds.Reserve(InEpicAccountIds.Num());
			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				EOS_Connect_GetExternalAccountMappingsOptions Options = {};
				Options.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_GETEXTERNALACCOUNTMAPPING_API_LATEST, 1);
				Options.LocalUserId = Data->LocalUserId;
				Options.AccountIdType = EOS_EExternalAccountType::EOS_EAT_EPIC;

				for (const EOS_EpicAccountId EpicAccountId : InEpicAccountIds)
				{
					FAccountId AccountId = FindAccountId(EpicAccountId);
					if (!AccountId.IsValid())
					{
						FEpicAccountIdStrBuffer EpicAccountIdStr;
						int32_t BufferSize = sizeof(EpicAccountIdStr);
						verify(EOS_EpicAccountId_ToString(EpicAccountId, EpicAccountIdStr.GetData(), &BufferSize) == EOS_EResult::EOS_Success);
						Options.TargetExternalUserId = &EpicAccountIdStr[0];
						const EOS_ProductUserId ProductUserId = EOS_Connect_GetExternalAccountMapping(ConnectHandle, &Options);
						AccountId = CreateAccountId(EpicAccountId, ProductUserId);
					}
					AccountIds.Emplace(MoveTemp(AccountId));
				}
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "ResolveAccountId failed to query external mapping Result=[%ls]", *LexToString(Data->ResultCode));
			}
		}
		Promise.SetValue(MoveTemp(AccountIds));
	});

	return Future;
}

TFuture<TArray<FAccountId>> FAuthEOS::ResolveAccountIds(const FAccountId& LocalAccountId, const TArray<EOS_ProductUserId>& InProductUserIds)
{
	// Search for all the account id's
	TArray<FAccountId> AccountIdHandles;
	AccountIdHandles.Reserve(InProductUserIds.Num());
	TArray<EOS_ProductUserId> MissingProductUserIds;
	MissingProductUserIds.Reserve(InProductUserIds.Num());
	for (const EOS_ProductUserId ProductUserId : InProductUserIds)
	{
		if (!EOS_ProductUserId_IsValid(ProductUserId))
		{
			return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
		}

		FAccountId Found = FindAccountId(ProductUserId);
		if (!Found.IsValid())
		{
			MissingProductUserIds.Emplace(ProductUserId);
		}
		AccountIdHandles.Emplace(MoveTemp(Found));
	}
	if (MissingProductUserIds.IsEmpty())
	{
		// We have them all, so we can just return
		return MakeFulfilledPromise<TArray<FAccountId>>(MoveTemp(AccountIdHandles)).GetFuture();
	}

	// If we failed to find all the handles, we need to query, which requires a valid LocalAccountId
	if (!(IsRunningDedicatedServer() || FOnlineAccountIdRegistryEOS::ValidateOnlineId(LocalAccountId)))
	{
		checkNoEntry();
		return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
	}

	TPromise<TArray<FAccountId>> Promise;
	TFuture<TArray<FAccountId>> Future = Promise.GetFuture();

	EOS_Connect_QueryProductUserIdMappingsOptions Options = {};
	Options.ApiVersion = 2;
	UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_QUERYPRODUCTUSERIDMAPPINGS_API_LATEST, 2);
	Options.LocalUserId = IsRunningDedicatedServer() ? nullptr : GetProductUserIdChecked(LocalAccountId);
	Options.ProductUserIds = MissingProductUserIds.GetData();
	Options.ProductUserIdCount = MissingProductUserIds.Num();

	TSharedPtr<IEOSFastTickLock> EOSSDKFastTickLock = GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle()->GetFastTickLock();

	EOS_Async(EOS_Connect_QueryProductUserIdMappings, ConnectHandle, Options,
	[this, WeakThis = AsWeak(), InProductUserIds, Promise = MoveTemp(Promise), EOSSDKFastTickLock = MoveTemp(EOSSDKFastTickLock)](const EOS_Connect_QueryProductUserIdMappingsCallbackInfo* Data) mutable -> void
	{
		TArray<FAccountId> AccountIds;
		if (const TSharedPtr<IAuth> StrongThis = WeakThis.Pin())
		{
			if (Data->ResultCode == EOS_EResult::EOS_Success)
			{
				EOS_Connect_GetProductUserIdMappingOptions Options = {};
				Options.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_GETPRODUCTUSERIDMAPPING_API_LATEST, 1);
				Options.LocalUserId = Data->LocalUserId;
				Options.AccountIdType = EOS_EExternalAccountType::EOS_EAT_EPIC;

				for (const EOS_ProductUserId ProductUserId : InProductUserIds)
				{
					FAccountId AccountId = FindAccountId(ProductUserId);
					if (!AccountId.IsValid())
					{
						Options.TargetProductUserId = ProductUserId;
						FEpicAccountIdStrBuffer EpicAccountIdStr;
						int32_t BufferLength = sizeof(EpicAccountIdStr);
						EOS_EpicAccountId EpicAccountId = nullptr;
						const EOS_EResult Result = EOS_Connect_GetProductUserIdMapping(ConnectHandle, &Options, EpicAccountIdStr.GetData(), &BufferLength);
						if (Result == EOS_EResult::EOS_Success)
						{
							EpicAccountId = EOS_EpicAccountId_FromString(EpicAccountIdStr.GetData());
							check(EOS_EpicAccountId_IsValid(EpicAccountId));
						}
						AccountId = CreateAccountId(EpicAccountId, ProductUserId);
					}
					AccountIds.Emplace(MoveTemp(AccountId));
				}
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "ResolveAccountId failed to query external mapping Result=[%ls]", *LexToString(Data->ResultCode));
			}
		}

		Promise.SetValue(MoveTemp(AccountIds));
	});

	return Future;
}

FAccountId FAuthEOS::CreateAccountId(const EOS_EpicAccountId EpicAccountId, const EOS_ProductUserId ProductUserId)
{
	return FOnlineAccountIdRegistryEOS::Get().FindOrAddAccountId(EpicAccountId, ProductUserId);
}

FAccountId FAuthEOS::FindAccountId(const EOS_ProductUserId ProductUserId)
{
	return UE::Online::FindAccountId(Services.GetServicesProvider(), ProductUserId);
}

FAccountId FAuthEOS::FindAccountId(const EOS_EpicAccountId EpicAccountId)
{
	return UE::Online::FindAccountId(Services.GetServicesProvider(), EpicAccountId);
}

/* UE::Online */ }
