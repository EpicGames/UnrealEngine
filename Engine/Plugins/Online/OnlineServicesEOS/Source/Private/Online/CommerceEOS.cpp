// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/CommerceEOS.h"

#include "IEOSSDKManager.h"
#include "Online/AuthEOS.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOS.h"
#include "Online/OnlineServicesEOS.h"

namespace UE::Online {

void FCommerceEOS::PostInitialize()
{	
	Super::PostInitialize();

	EcomHandle = EOS_Platform_GetEcomInterface(*GetServices<FOnlineServicesEOS>().GetEOSPlatformHandle());
	check(EcomHandle);
}

void FCommerceEOS::PreShutdown()
{
	Super::PreShutdown();
}

void FCommerceEOS::EOSOfferToOssOffer(FOffer& OutOffer, EOS_Ecom_CatalogOffer* EosOffer)
{
	OutOffer.OfferId = UTF8_TO_TCHAR(EosOffer->Id);
	OutOffer.Title = FText::FromString(UTF8_TO_TCHAR(EosOffer->TitleText));
	OutOffer.LongDescription = FText::FromString(UTF8_TO_TCHAR(EosOffer->LongDescriptionText));
	OutOffer.ExpirationDate = FDateTime(EosOffer->ExpirationTimestamp);
	OutOffer.CurrencyCode = UTF8_TO_TCHAR(EosOffer->CurrencyCode);
	OutOffer.PurchaseLimit = EosOffer->PurchaseLimit;
	if (EosOffer->PriceResult == EOS_EResult::EOS_Success)
	{
		OutOffer.RegularPrice = EosOffer->OriginalPrice64;
		OutOffer.Price = EosOffer->CurrentPrice64;
		OutOffer.PriceDecimalPoint = EosOffer->DecimalPoint;

		OutOffer.FormattedPrice = GetFormattedPrice(OutOffer.Price, OutOffer.PriceDecimalPoint, OutOffer.CurrencyCode);
		OutOffer.FormattedRegularPrice = GetFormattedPrice(OutOffer.RegularPrice, OutOffer.PriceDecimalPoint, OutOffer.CurrencyCode);
	}
	// Other properties that could potentially be added in the future- image data (EOS_Ecom_KeyImageInfo ), individual items (EOS_Ecom_CatalogItem)
}

void FCommerceEOS::EOSEntitlementToOssEntitlement(FEntitlement& OutEntitlement, EOS_Ecom_Entitlement* EosEntitlement)
{
	OutEntitlement.EntitlementId = UTF8_TO_TCHAR(EosEntitlement->EntitlementId);
	OutEntitlement.bRedeemed = (EosEntitlement->bRedeemed == EOS_TRUE);
	OutEntitlement.ProductId = UTF8_TO_TCHAR(EosEntitlement->CatalogItemId);
	OutEntitlement.Quantity = 1;
	// do we have a suitable placement for Name or AcquiredDate here?
}

TOnlineAsyncOpHandle<FCommerceQueryOffers> FCommerceEOS::QueryOffers(FCommerceQueryOffers::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryOffers> Op = GetJoinableOp<FCommerceQueryOffers>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& Op)
		{
			const FCommerceQueryOffers::Params& Params = Op.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
			EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
			if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
		})
		.Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& Op, TPromise<const EOS_Ecom_QueryOffersCallbackInfo*>&& Promise)
		{
			const FCommerceQueryOffers::Params& Params = Op.GetParams();
			EOS_Ecom_QueryOffersOptions Options = { };
			Options.ApiVersion = 1;
			FTCHARToUTF8 Utf8OverrideCatalogNamespace(*Params.OverrideCatalogNamespace);
			if (!Params.OverrideCatalogNamespace.IsEmpty()) {
				Options.OverrideCatalogNamespace = Utf8OverrideCatalogNamespace.Get();
			}
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOFFERS_API_LATEST, 1);
			Options.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
			EOS_Async(EOS_Ecom_QueryOffers, EcomHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& Op, const EOS_Ecom_QueryOffersCallbackInfo* Data)
		{
			const EOS_EResult Result = Data->ResultCode;
			if (Result != EOS_EResult::EOS_Success)
			{
				Op.SetError(Errors::FromEOSResult(Result));
				return;
			}

			EOS_Ecom_GetOfferCountOptions CountOptions = { };
			CountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_GETOFFERCOUNT_API_LATEST, 1);
			CountOptions.LocalUserId = Data->LocalUserId;
			uint32 OfferCount = EOS_Ecom_GetOfferCount(EcomHandle, &CountOptions);

			TArray<FOffer>& Offers = CachedOffers.FindOrAdd(Op.GetParams().LocalAccountId, TArray<FOffer>());
			Offers.Empty(OfferCount);

			EOS_Ecom_CopyOfferByIndexOptions OfferOptions = { };
			OfferOptions.ApiVersion = 3;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYOFFERBYINDEX_API_LATEST, 3);
			OfferOptions.LocalUserId = Data->LocalUserId;
			// Iterate and parse the offer list
			for (uint32 OfferIndex = 0; OfferIndex < OfferCount; OfferIndex++)
			{
				EOS_Ecom_CatalogOffer* EosOffer = nullptr;
				OfferOptions.OfferIndex = OfferIndex;
				const EOS_EResult OfferResult = EOS_Ecom_CopyOfferByIndex(EcomHandle, &OfferOptions, &EosOffer);
				if (OfferResult != EOS_EResult::EOS_Success)
				{
					continue;
				}
				FOffer& Offer = Offers.Emplace_GetRef();
				EOSOfferToOssOffer(Offer, EosOffer);
				EOS_Ecom_CatalogOffer_Release(EosOffer);
			}

			Op.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryOffersById> FCommerceEOS::QueryOffersById(FCommerceQueryOffersById::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryOffersById> Op = GetJoinableOp<FCommerceQueryOffersById>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		// EOS doesn't support anything ID-specific, just use the generic QueryOffers call
		QueryOffers({ Op->GetParams().LocalAccountId })
			.OnComplete([Op](const TOnlineResult<FCommerceQueryOffers>& Result)
		{
			if (Result.IsOk())
			{
				Op->SetResult({});
			}
			else
			{
				FOnlineError Error = Result.GetErrorValue();
				Op->SetError(MoveTemp(Error));
			}
		});
	}
	return Op->GetHandle();
}

TOnlineResult<FCommerceGetOffers> FCommerceEOS::GetOffers(FCommerceGetOffers::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetOffers>(Errors::NotLoggedIn());
	}

	if(CachedOffers.Contains(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetOffers>({CachedOffers.FindChecked(Params.LocalAccountId)});
	}
	return TOnlineResult<FCommerceGetOffers>(Errors::NotFound());
}

TOnlineResult<FCommerceGetOffersById> FCommerceEOS::GetOffersById(FCommerceGetOffersById::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetOffersById>(Errors::NotLoggedIn());
	}

	if (CachedOffers.Contains(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetOffersById>({ CachedOffers.FindChecked(Params.LocalAccountId).FilterByPredicate(
			[&Params](const FOffer& Offer)
			{ 
				return Params.OfferIds.Contains(Offer.OfferId);
			})
		});
	}
	return TOnlineResult<FCommerceGetOffersById>(Errors::NotFound());
}

TOnlineAsyncOpHandle<FCommerceShowStoreUI> FCommerceEOS::ShowStoreUI(FCommerceShowStoreUI::Params&& Params)
{
	// Todo: Implement when EOS store overlay is fully supported
	TOnlineAsyncOpRef<FCommerceShowStoreUI> Op = GetOp<FCommerceShowStoreUI>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceCheckout> FCommerceEOS::Checkout(FCommerceCheckout::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceCheckout> Op = GetOp<FCommerceCheckout>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FCommerceCheckout>& Op, TPromise<const EOS_Ecom_CheckoutCallbackInfo*>&& Promise)
	{
		const FCommerceCheckout::Params& Params = Op.GetParams();
		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			Promise.SetValue(nullptr);
			return;
		}
		EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
		if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
		{
			Op.SetError(Errors::NotLoggedIn());
			Promise.SetValue(nullptr);
			return;
		}

		TArray<FTCHARToUTF8> Utf8CheckoutIds;
		Utf8CheckoutIds.Reserve(Params.Offers.Num());

		TArray<EOS_Ecom_CheckoutEntry> EosCheckoutEntries;
		EosCheckoutEntries.Reserve(Params.Offers.Num());

		for (const FPurchaseOffer& Offer : Params.Offers)
		{
			Utf8CheckoutIds.Emplace(*Offer.OfferId);

			EOS_Ecom_CheckoutEntry& CheckoutEntry = EosCheckoutEntries.AddDefaulted_GetRef();
			CheckoutEntry.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUTENTRY_API_LATEST, 1);
			CheckoutEntry.OfferId = Utf8CheckoutIds.Last().Get();
		}

		EOS_Ecom_CheckoutOptions Options = { };
		Options.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_CHECKOUT_API_LATEST, 2);
		Options.LocalUserId = LocalUserEasId;
		Options.EntryCount = Params.Offers.Num();
		Options.Entries = EosCheckoutEntries.GetData();
		Options.OverrideCatalogNamespace = nullptr;
		Options.PreferredOrientation = EOS_ECheckoutOrientation::EOS_ECO_Default;

		EOS_Async(EOS_Ecom_Checkout, EcomHandle, Options, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FCommerceCheckout>& Op, const EOS_Ecom_CheckoutCallbackInfo* Data)
	{
		EOS_EResult Result = Data->ResultCode;
		if (Result != EOS_EResult::EOS_Success)
		{
			if (Result == EOS_EResult::EOS_Canceled)
			{
				UE_LOGF(LogOnlineServices, Log, "EOS_Ecom_Checkout was canceled by user.");
			}
			else if (Result == EOS_EResult::EOS_Ecom_PurchaseProcessing)
			{
				// See header documentation for EOS_Ecom_PurchaseProcessing and online EOS documentation for more information
				UE_LOGF(LogOnlineServices, Log, "QueryEntitlements will need to be called to confirm the purchase. This can be done via a UI button.");
			}
			else
			{
				UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_Checkout: failed with error (%ls)", *LexToString(Data->ResultCode));
			}
			Op.SetError(Errors::FromEOSResult(Result));
			return;
		}

		FCommerceCheckout::Result ResultValue;
		ResultValue.TransactionId.Emplace(ANSI_TO_TCHAR(Data->TransactionId));
		Op.SetResult(MoveTemp(ResultValue));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> FCommerceEOS::QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryTransactionEntitlements> Op = GetJoinableOp<FCommerceQueryTransactionEntitlements>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FCommerceQueryTransactionEntitlements>& Op)
		{
			const FCommerceQueryTransactionEntitlements::Params& Params = Op.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
			EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
			if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}

			EOS_Ecom_HTransaction OutTransaction;
			FTCHARToUTF8 Utf8TransactionId(*Params.TransactionId);
			EOS_Ecom_CopyTransactionByIdOptions CopyTransactionOptions = { };
			CopyTransactionOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYTRANSACTIONBYID_API_LATEST, 1);
			CopyTransactionOptions.LocalUserId = LocalUserEasId;
			CopyTransactionOptions.TransactionId = Utf8TransactionId.Get();

			EOS_EResult CopyTransactionResult = EOS_Ecom_CopyTransactionById(EcomHandle, &CopyTransactionOptions, &OutTransaction);
			if (CopyTransactionResult != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_CopyTransactionById: failed with error (%ls)", *LexToString(CopyTransactionResult));
				Op.SetError(Errors::FromEOSResult(CopyTransactionResult));
				return;
			}

			EOS_Ecom_Transaction_GetEntitlementsCountOptions EntitlementCountOptions = { };
			EntitlementCountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_TRANSACTION_GETENTITLEMENTSCOUNT_API_LATEST, 1);
			int32 NumTransactionsFound = EOS_Ecom_Transaction_GetEntitlementsCount(OutTransaction, &EntitlementCountOptions);

			if (NumTransactionsFound > 0)
			{
				FCommerceQueryTransactionEntitlements::Result OutResult;

				for (int32 i = 0; i < NumTransactionsFound; i++)
				{
					EOS_Ecom_Entitlement* EosEntitlement;
					EOS_Ecom_Transaction_CopyEntitlementByIndexOptions CopyTransactionEntitlementOptions = {};
					CopyTransactionEntitlementOptions.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_TRANSACTION_COPYENTITLEMENTBYINDEX_API_LATEST, 1);
					CopyTransactionEntitlementOptions.EntitlementIndex = i;

					EOS_EResult Result = EOS_Ecom_Transaction_CopyEntitlementByIndex(OutTransaction, &CopyTransactionEntitlementOptions, &EosEntitlement);
					if (Result == EOS_EResult::EOS_Success)
					{
						FEntitlement& Entitlement = OutResult.Entitlements.Emplace_GetRef();
						EOSEntitlementToOssEntitlement(Entitlement, EosEntitlement);
						EOS_Ecom_Entitlement_Release(EosEntitlement);
					}
				}

				Op.SetResult(MoveTemp(OutResult));
			}
			else
			{
				Op.SetError(Errors::NotFound());
			}
			EOS_Ecom_Transaction_Release(OutTransaction);
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryEntitlements> FCommerceEOS::QueryEntitlements(FCommerceQueryEntitlements::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryEntitlements> Op = GetJoinableOp<FCommerceQueryEntitlements>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op)
		{
			const FCommerceQueryEntitlements::Params& Params = Op.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
			EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
			if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
		})
		.Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op, TPromise<const EOS_Ecom_QueryEntitlementsCallbackInfo*>&& Promise)
		{
			const FCommerceQueryEntitlements::Params& Params = Op.GetParams();
			EOS_Ecom_QueryEntitlementsOptions Options = { };
			Options.ApiVersion = 3;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYENTITLEMENTS_API_LATEST, 3);
			Options.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
			Options.bIncludeRedeemed = EOS_TRUE;
			Options.OverrideCatalogNamespace = nullptr;
			EOS_Async(EOS_Ecom_QueryEntitlements, EcomHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op, const EOS_Ecom_QueryEntitlementsCallbackInfo* Data)
		{
			const EOS_EResult Result = Data->ResultCode;
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_QueryEntitlements: failed with error (%ls)", *LexToString(Data->ResultCode));
				Op.SetError(Errors::FromEOSResult(Result));
				return;
			}

			EOS_Ecom_GetEntitlementsCountOptions CountOptions = { };
			CountOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_GETENTITLEMENTSCOUNT_API_LATEST, 1);
			CountOptions.LocalUserId = Data->LocalUserId;
			uint32 Count = EOS_Ecom_GetEntitlementsCount(EcomHandle, &CountOptions);
			TArray<FEntitlement>& Entitlements = CachedEntitlements.FindOrAdd(Op.GetParams().LocalAccountId, TArray<FEntitlement>());
			Entitlements.Reset(Count);

			UE_LOGF(LogOnlineServices, VeryVerbose, "Entitlements Count: %d", Count);

			EOS_Ecom_CopyEntitlementByIndexOptions CopyOptions = { };
			CopyOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_COPYENTITLEMENTBYINDEX_API_LATEST, 1);
			CopyOptions.LocalUserId = Data->LocalUserId;

			for (uint32 Index = 0; Index < Count; Index++)
			{
				CopyOptions.EntitlementIndex = Index;

				UE_LOGF(LogOnlineServices, VeryVerbose, "	Entitlement %d:", Index);

				EOS_Ecom_Entitlement* EosEntitlement = nullptr;
				EOS_EResult CopyResult = EOS_Ecom_CopyEntitlementByIndex(EcomHandle, &CopyOptions, &EosEntitlement);
				if (CopyResult != EOS_EResult::EOS_Success && CopyResult != EOS_EResult::EOS_Ecom_EntitlementStale)
				{
					UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_CopyEntitlementByIndex: failed with error (%ls) (proceeding with operation)", *LexToString(CopyResult));
					continue;
				}

				// Parse the entitlement into the receipt format
				FEntitlement& Entitlement = Entitlements.Emplace_GetRef();
				EOSEntitlementToOssEntitlement(Entitlement, EosEntitlement);

				UE_LOGF(LogOnlineServices, VeryVerbose, "		Entitlement Id: %ls", *Entitlement.EntitlementId);
				UE_LOGF(LogOnlineServices, VeryVerbose, "		Product Id: %ls", *Entitlement.ProductId);

				EOS_Ecom_Entitlement_Release(EosEntitlement);
			}
		})
		.Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op, TPromise<const EOS_Ecom_QueryOwnershipBySandboxIdsCallbackInfo*>&& Promise)
		{
			const FCommerceQueryEntitlements::Params& Params = Op.GetParams();

			EOS_Ecom_QueryOwnershipBySandboxIdsOptions Options = {};
			Options.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOWNERSHIPBYSANDBOXIDSOPTIONS_API_LATEST, 1);
			Options.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
			Options.SandboxIdsCount = 1;

			if (IEOSSDKManager* Manager = IEOSSDKManager::Get())
			{
				const FString& PlatformConfigName = StaticCast<FOnlineServicesEpicCommon*>(&Services)->GetEOSPlatformHandle()->GetConfigName();
				if (const FEOSSDKPlatformConfig* Config = Manager->GetPlatformConfig(PlatformConfigName))
				{
					const FTCHARToUTF8& Converter = FTCHARToUTF8(*Config->SandboxId);
					const char* SandboxId = Converter.Get();
					Options.SandboxIds = &SandboxId;
				}
			}
			
			EOS_Async(EOS_Ecom_QueryOwnershipBySandboxIds, EcomHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op, const EOS_Ecom_QueryOwnershipBySandboxIdsCallbackInfo* Data)
		{
			const EOS_EResult Result = Data->ResultCode;
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_QueryOwnership: failed with error (%ls)", *LexToString(Data->ResultCode));
				Op.SetError(Errors::FromEOSResult(Result));
				return;
			}

			const FCommerceQueryEntitlements::Params& Params = Op.GetParams();
			
			TArray<FEntitlement>& Entitlements = CachedEntitlements.FindOrAdd(Params.LocalAccountId, TArray<FEntitlement>());
			
			UE_LOGF(LogOnlineServices, VeryVerbose, "Sandbox Id Ownership Count: %d", Data->SandboxIdItemOwnershipsCount);

			for (const EOS_Ecom_SandboxIdItemOwnership& SandboxOwnershipEntry : TArrayView<const EOS_Ecom_SandboxIdItemOwnership>(Data->SandboxIdItemOwnerships, Data->SandboxIdItemOwnershipsCount))
			{
				UE_LOGF(LogOnlineServices, VeryVerbose, "	Sandbox Id: %s", SandboxOwnershipEntry.SandboxId);
				UE_LOGF(LogOnlineServices, VeryVerbose, "	Ownership Count: %d", SandboxOwnershipEntry.OwnedCatalogItemIdsCount);

				for (uint32_t CatalogItemIndex = 0; CatalogItemIndex < SandboxOwnershipEntry.OwnedCatalogItemIdsCount; CatalogItemIndex++)
				{
					const EOS_Ecom_CatalogItemId& CatalogItemId = SandboxOwnershipEntry.OwnedCatalogItemIds[CatalogItemIndex];

					FEntitlement& Entitlement = Entitlements.Emplace_GetRef();
					Entitlement.bRedeemed = true;
					Entitlement.ProductId = FString(CatalogItemId);
					Entitlement.Quantity = 1;

					UE_LOGF(LogOnlineServices, VeryVerbose, "		Catalog Item %d", CatalogItemIndex);
					UE_LOGF(LogOnlineServices, VeryVerbose, "			Catalog Item Id: %s", CatalogItemId);
				}
			}

			Op.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}

	return Op->GetHandle();
}

TOnlineResult<FCommerceGetEntitlements> FCommerceEOS::GetEntitlements(FCommerceGetEntitlements::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetEntitlements>(Errors::NotLoggedIn());
	}

	if (!CachedEntitlements.Contains(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetEntitlements>(Errors::NotFound());
	}
	return TOnlineResult<FCommerceGetEntitlements>({CachedEntitlements.FindChecked(Params.LocalAccountId)});
}

TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> FCommerceEOS::RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceRedeemEntitlement> Op = GetOp<FCommerceRedeemEntitlement>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FCommerceRedeemEntitlement>& Op, TPromise<const EOS_Ecom_RedeemEntitlementsCallbackInfo*>&& Promise)
	{
		const FCommerceRedeemEntitlement::Params& Params = Op.GetParams();
		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			Promise.SetValue(nullptr);
			return;
		}
		EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
		if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
		{
			Op.SetError(Errors::NotLoggedIn());
			Promise.SetValue(nullptr);
			return;
		}
		if (Params.Quantity != 1)
		{
			Op.SetError(Errors::InvalidParams());
			Promise.SetValue(nullptr);
			return;
		}

		FTCHARToUTF8 Utf8EntitlementId(*Op.GetParams().EntitlementId);
		const char* EntitlementIdPtr = Utf8EntitlementId.Get();
		EOS_Ecom_RedeemEntitlementsOptions Options = { };
		Options.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_REDEEMENTITLEMENTS_API_LATEST, 2);
		Options.LocalUserId = LocalUserEasId;
		Options.EntitlementIdCount = 1;
		Options.EntitlementIds = &EntitlementIdPtr;
		
		EOS_Async(EOS_Ecom_RedeemEntitlements, EcomHandle, Options, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FCommerceRedeemEntitlement>& Op, const EOS_Ecom_RedeemEntitlementsCallbackInfo* Data)
	{
		EOS_EResult Result = Data->ResultCode;
		if (Result != EOS_EResult::EOS_Success)
		{
			UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_RedeemEntitlements: failed with error (%ls)", *LexToString(Data->ResultCode));
			Op.SetError(Errors::FromEOSResult(Result));
			return;
		}
		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> FCommerceEOS::RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceRetrieveS2SToken> Op = GetJoinableOp<FCommerceRetrieveS2SToken>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FCommerceRetrieveS2SToken>& Op)
		{
			const FCommerceRetrieveS2SToken::Params& Params = Op.GetParams();
			if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
			EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
			if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}
		})
		.Then([this](TOnlineAsyncOp<FCommerceRetrieveS2SToken>& Op, TPromise<const EOS_Ecom_QueryOwnershipTokenCallbackInfo*>&& Promise)
		{
			const FCommerceRetrieveS2SToken::Params& Params = Op.GetParams();
			EOS_Ecom_QueryOwnershipTokenOptions Options = {};
			Options.ApiVersion = 2;
			UE_EOS_CHECK_API_MISMATCH(EOS_ECOM_QUERYOWNERSHIPTOKEN_API_LATEST, 2);
			Options.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
			EOS_Async(EOS_Ecom_QueryOwnershipToken, EcomHandle, Options, MoveTemp(Promise));
		})
		.Then([this](TOnlineAsyncOp<FCommerceRetrieveS2SToken>& Op, const EOS_Ecom_QueryOwnershipTokenCallbackInfo* Data)
		{
			EOS_EResult Result = Data->ResultCode;
			if (Result != EOS_EResult::EOS_Success)
			{
				UE_LOGF(LogOnlineServices, Error, "EOS_Ecom_QueryOwnershipToken: failed with error (%ls)", *LexToString(Data->ResultCode));
				Op.SetError(Errors::FromEOSResult(Result));
				return;
			}

			size_t DataSize = strlen(Data->OwnershipToken);
			FCommerceRetrieveS2SToken::Result ResultValue;
			ResultValue.Token.AddZeroed(DataSize);
			for (size_t i = 0; i < DataSize; i++)
			{
				ResultValue.Token[i] = (uint8)Data->OwnershipToken[i];
			}
			Op.SetResult(MoveTemp(ResultValue));
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

} // namespace UE::Online
