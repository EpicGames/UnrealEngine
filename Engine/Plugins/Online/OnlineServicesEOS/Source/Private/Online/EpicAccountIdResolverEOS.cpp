// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/EpicAccountIdResolverEOS.h"

#include "Online/AuthEOS.h"
#include "Online/OnlineServicesEOS.h"

namespace UE::Online {

FEpicAccountIdResolverEOS::FEpicAccountIdResolverEOS(FOnlineServicesEOS& InServices)
: TOnlineComponent(TEXT("EpicAccountIdResolver"), InServices)
{
}

TFuture<FAccountId> FEpicAccountIdResolverEOS::ResolveAccountId(const FAccountId LocalAccountId, const EOS_EpicAccountId EpicAccountId)
{
	TPromise<FAccountId> Promise;
	TFuture<FAccountId> Future = Promise.GetFuture();

	Services.Get<FAuthEOS>()->ResolveAccountIds(LocalAccountId, { EpicAccountId }).Next([Promise = MoveTemp(Promise)](const TArray<FAccountId>& AccountIds) mutable
	{
		FAccountId Result;
		if (AccountIds.Num() == 1)
		{
			Result = AccountIds[0];
		}
		Promise.SetValue(Result);
	});

	return Future;
}

TFuture<TArray<FAccountId>> FEpicAccountIdResolverEOS::ResolveAccountIds(const FAccountId LocalAccountId, const TArray<EOS_EpicAccountId>& EpicAccountIds)
{
	return Services.Get<FAuthEOS>()->ResolveAccountIds(LocalAccountId, EpicAccountIds);
}

/* UE::Online */ }
