// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/EpicProductUserIdResolverEOS.h"

#include "Online/AuthEOS.h"
#include "Online/OnlineServicesEOS.h"

namespace UE::Online {

FEpicProductUserIdResolverEOS::FEpicProductUserIdResolverEOS(FOnlineServicesEOS& InServices)
: TOnlineComponent(TEXT("EpicProductUserIdResolver"), InServices)
{
}

TFuture<FAccountId> FEpicProductUserIdResolverEOS::ResolveAccountId(const FAccountId LocalAccountId, const EOS_ProductUserId ProductUserId)
{
	TPromise<FAccountId> Promise;
	TFuture<FAccountId> Future = Promise.GetFuture();

	Services.Get<FAuthEOS>()->ResolveAccountIds(LocalAccountId, { ProductUserId }).Next([Promise = MoveTemp(Promise)](const TArray<FAccountId>& AccountIds) mutable
	{
		FAccountId Result;
		if (AccountIds.Num() == 1)
		{
			Result = AccountIds[0];
		}
		Promise.SetValue(Result);
	});

	return Future;
}

TFuture<TArray<FAccountId>> FEpicProductUserIdResolverEOS::ResolveAccountIds(const FAccountId LocalAccountId, const TArray<EOS_ProductUserId>& ProductUserIds)
{
	return Services.Get<FAuthEOS>()->ResolveAccountIds(LocalAccountId, ProductUserIds);
}

/* UE::Online */ }
