// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/ExternalUIEOS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/OnlineIdEOS.h"
#include "Online/OnlineServicesEOS.h"
#include "Online/OnlineServicesEOSGSTypes.h"

#include "eos_ui.h"

namespace UE::Online {

FExternalUIEOS::FExternalUIEOS(FOnlineServicesEpicCommon& InServices)
	: Super(InServices)
{
}

FExternalUIEOS::~FExternalUIEOS() = default;

void FExternalUIEOS::Initialize()
{
	Super::Initialize();

	UIHandle = EOS_Platform_GetUIInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(UIHandle != nullptr);
}

void FExternalUIEOS::UpdateConfig()
{
	Super::UpdateConfig();
	LoadConfig(Config);
}

TOnlineAsyncOpHandle<FExternalUIShowLoginUI> FExternalUIEOS::ShowLoginUI(FExternalUIShowLoginUI::Params&& Params)
{
	TOnlineAsyncOpRef<FExternalUIShowLoginUI> Op = GetOp<FExternalUIShowLoginUI>(MoveTemp(Params));
	if (Config.bShowLoginUIEnabled)
	{
		// Use account portal auth login
		Op->Then([this](TOnlineAsyncOp<FExternalUIShowLoginUI>& InAsyncOp)
		{
			FAuthLogin::Params AuthLoginParams;
			AuthLoginParams.PlatformUserId = InAsyncOp.GetParams().PlatformUserId;
			AuthLoginParams.CredentialsType = TEXT("accountportal");
			AuthLoginParams.Scopes = InAsyncOp.GetParams().Scopes;

			TPromise<void>* Promise = new TPromise<void>();
			TFuture<void> Future = Promise->GetFuture();
			GetServices().GetAuthInterface()->Login(MoveTemp(AuthLoginParams))
				.OnComplete([Op = InAsyncOp.AsShared(), Promise](const TOnlineResult<FAuthLogin>& AuthLoginResult) mutable
				{
					if (AuthLoginResult.IsOk())
					{
						FExternalUIShowLoginUI::Result ExtUIShowLoginResult = { AuthLoginResult.GetOkValue().AccountInfo };
						Op->SetResult(CopyTemp(ExtUIShowLoginResult));
					}
					else
					{
						Op->SetError(CopyTemp(AuthLoginResult.GetErrorValue()));
					}
					Promise->SetValue();
					delete Promise;
				});
			return Future;
		}).Enqueue(GetSerialQueue());
	}
	else
	{
		Op->SetError(Errors::NotImplemented());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> FExternalUIEOS::ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params)
{
	TOnlineAsyncOpRef<FExternalUIShowFriendsUI> Op = GetOp<FExternalUIShowFriendsUI>(MoveTemp(Params));

	EOS_EpicAccountId LocalUserEasId = GetEpicAccountId(Params.LocalAccountId);
	if (!EOS_EpicAccountId_IsValid(LocalUserEasId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[FExternalUIEOS::ShowFriendsUI] LocalAccountId=[%ls] EpicAccountId not found", *ToLogString(Params.LocalAccountId));
		Op->SetError(Errors::Unknown()); // TODO
		return Op->GetHandle();
	}

	EOS_UI_ShowFriendsOptions ShowFriendsOptions = {};
	ShowFriendsOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_UI_SHOWFRIENDS_API_LATEST, 1);
	ShowFriendsOptions.LocalUserId = LocalUserEasId;

	Op->Then([this, ShowFriendsOptions](TOnlineAsyncOp<FExternalUIShowFriendsUI>& InAsyncOp, TPromise<const EOS_UI_ShowFriendsCallbackInfo*>&& Promise) mutable
	{
		EOS_Async(EOS_UI_ShowFriends, UIHandle, ShowFriendsOptions, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FExternalUIShowFriendsUI>& InAsyncOp, const EOS_UI_ShowFriendsCallbackInfo* Data)
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FExternalUIEOS::ShowFriendsUI] EOS_UI_ShowFriends Result [%ls] for User [%ls].", *LexToString(Data->ResultCode), *LexToString(Data->LocalUserId));

		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			FExternalUIShowFriendsUI::Result Result = {};
			InAsyncOp.SetResult(MoveTemp(Result));
		}
		else
		{
			InAsyncOp.SetError(Errors::FromEOSResult(Data->ResultCode));
		}
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

/* UE::Online */ }
