// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdEOS.h"

#include "EOSShared.h"
#include "String/HexToBytes.h"
#include "String/BytesToHex.h"

namespace UE::Online {

FString FOnlineAccountIdRegistryEOS::ToString(const FAccountId& AccountId) const
{
	FString Result;
	if (ValidateOnlineId(AccountId))
	{
		const FOnlineAccountIdDataEOS& IdData = GetAccountIdData(AccountId);
		Result = FString::Printf(TEXT("%s|%s"), *LexToString(IdData.EpicAccountId), *LexToString(IdData.ProductUserId));
	}
	else
	{
		check(!AccountId.IsValid()); // Check we haven't been passed a valid handle for a different EOnlineServices.
		Result = TEXT("Invalid");
	}
	return Result;
}

FString FOnlineAccountIdRegistryEOS::ToLogString(const FAccountId& AccountId) const
{
	FString Result;
	if (ValidateOnlineId(AccountId))
	{
		const FOnlineAccountIdDataEOS& IdData = GetAccountIdData(AccountId);
		Result = FString::Printf(TEXT("EAS=[%s] EOS=[%s]"), *LexToString(IdData.EpicAccountId), *LexToString(IdData.ProductUserId));
	}
	else
	{
		check(!AccountId.IsValid()); // Check we haven't been passed a valid handle for a different EOnlineServices.
		Result = TEXT("Invalid");
	}
	return Result;
}

enum class EEOSAccountIdElements : uint8
{
	None = 0,
	EAS = 1 << 0,
	EOS = 1 << 1
};
ENUM_CLASS_FLAGS(EEOSAccountIdElements);
const uint8 OnlineIdEOSUtf8BufferLength = 32;
const uint8 OnlineIdEOSHexBufferLength = 16;

TArray<uint8> FOnlineAccountIdRegistryEOS::ToReplicationData(const FAccountId& AccountId) const
{
	TArray<uint8> ReplicationData;
	if (ValidateOnlineId(AccountId))
	{
		const FOnlineAccountIdDataEOS& IdData = GetAccountIdData(AccountId);
		EEOSAccountIdElements Elements = EEOSAccountIdElements::None;

		char EasBuffer[EOS_EPICACCOUNTID_MAX_LENGTH + 1] = { 0 };
		if (EOS_EpicAccountId_IsValid(IdData.EpicAccountId))
		{
			int32 EasBufferLength = sizeof(EasBuffer);
			const EOS_EResult EosResult = EOS_EpicAccountId_ToString(IdData.EpicAccountId, EasBuffer, &EasBufferLength);
			if (ensure(EosResult == EOS_EResult::EOS_Success))
			{
				check(EasBufferLength - 1 == OnlineIdEOSUtf8BufferLength);
				Elements |= EEOSAccountIdElements::EAS;
			}
		}

		char EosBuffer[EOS_PRODUCTUSERID_MAX_LENGTH + 1] = { 0 };
		if (EOS_ProductUserId_IsValid(IdData.ProductUserId))
		{
			int32 EosBufferLength = sizeof(EosBuffer);
			const EOS_EResult EosResult = EOS_ProductUserId_ToString(IdData.ProductUserId, EosBuffer, &EosBufferLength);
			if (ensure(EosResult == EOS_EResult::EOS_Success))
			{
				check(EosBufferLength - 1 == OnlineIdEOSUtf8BufferLength);
				Elements |= EEOSAccountIdElements::EOS;
			}
		}

		const int EasHexBufferLength = EnumHasAnyFlags(Elements, EEOSAccountIdElements::EAS) ? OnlineIdEOSHexBufferLength : 0;
		const int EosHexBufferLength = EnumHasAnyFlags(Elements, EEOSAccountIdElements::EOS) ? OnlineIdEOSHexBufferLength : 0;
		ReplicationData.SetNumUninitialized(1 + EasHexBufferLength + EosHexBufferLength);
		ReplicationData[0] = uint8(Elements);
		if (EasHexBufferLength > 0)
		{
			UE::String::HexToBytes(FUtf8StringView(EasBuffer, OnlineIdEOSUtf8BufferLength), ReplicationData.GetData() + 1);
		}
		if (EosHexBufferLength > 0)
		{
			UE::String::HexToBytes(FUtf8StringView(EosBuffer, OnlineIdEOSUtf8BufferLength), ReplicationData.GetData() + 1 + EasHexBufferLength);
		}
	}
	else
	{
		check(!AccountId.IsValid()); // Check we haven't been passed a valid handle for a different EOnlineServices.
	}
	return ReplicationData;
}

FAccountId FOnlineAccountIdRegistryEOS::FromReplicationData(const TArray<uint8>& ReplicationData)
{
	const EEOSAccountIdElements Elements = EEOSAccountIdElements(ReplicationData[0]);
	const int EasHexBufferLength = EnumHasAnyFlags(Elements, EEOSAccountIdElements::EAS) ? OnlineIdEOSHexBufferLength : 0;
	const int EosHexBufferLength = EnumHasAnyFlags(Elements, EEOSAccountIdElements::EOS) ? OnlineIdEOSHexBufferLength : 0;

	EOS_EpicAccountId EpicAccountId = nullptr;
	if (EasHexBufferLength > 0)
	{
		char EasBuffer[EOS_EPICACCOUNTID_MAX_LENGTH + 1] = { 0 };
		UE::String::BytesToHexLower(TConstArrayView<uint8>(ReplicationData.GetData() + 1, EasHexBufferLength), EasBuffer);
		EpicAccountId = EOS_EpicAccountId_FromString(EasBuffer);
	}

	EOS_ProductUserId ProductUserId = nullptr;
	if (EosHexBufferLength > 0)
	{
		char EosBuffer[EOS_EPICACCOUNTID_MAX_LENGTH + 1] = { 0 };
		UE::String::BytesToHex(TConstArrayView<uint8>(ReplicationData.GetData() + 1 + EasHexBufferLength, EosHexBufferLength), EosBuffer);
		ProductUserId = EOS_ProductUserId_FromString(EosBuffer);
	}

	return FindOrAddAccountId(EpicAccountId, ProductUserId);
}

FAccountId FOnlineAccountIdRegistryEOS::FromStringData(const FString& StringData)
{
	FString EpicAccountIdString;
	FString ProductUserIdString;

	StringData.Split(TEXT(":"), &EpicAccountIdString, &ProductUserIdString);

	EOS_EpicAccountId EpicAccountId = nullptr;
	if (!EpicAccountIdString.IsEmpty())
	{
		const auto Utf8EpicAccountId = StringCast<UTF8CHAR>(*EpicAccountIdString);
		EpicAccountId = EOS_EpicAccountId_FromString((const char*)Utf8EpicAccountId.Get());
	}

	EOS_ProductUserId ProductUserId = nullptr;
	if (!ProductUserIdString.IsEmpty())
	{
		const auto Utf8ProductUserId = StringCast<UTF8CHAR>(*ProductUserIdString);
		ProductUserId = EOS_ProductUserId_FromString((const char*)Utf8ProductUserId.Get());
	}

	return FindOrAddAccountId(EpicAccountId, ProductUserId);
}

FAccountId FOnlineAccountIdRegistryEOS::FindOrAddAccountId(const EOS_ProductUserId ProductUserId)
{
	checkNoEntry();

	UE_LOGF(LogTemp, Warning, "This method is not implemented in FOnlineAccountIdRegistryEOS, use FindOrAddAccountId(const EOS_EpicAccountId InEpicAccountId, const EOS_ProductUserId InProductUserId) instead.");

	return FAccountId();
}

FAccountId FOnlineAccountIdRegistryEOS::FindAccountId(const EOS_ProductUserId ProductUserId) const
{
	FAccountId AccountId;
	if (EOS_ProductUserId_IsValid(ProductUserId) == EOS_TRUE)
	{
		const UE::TReadScopeLock ReadLock(Lock);
		if (const FAccountId* Found = ProductUserIdToAccountId.Find(ProductUserId))
		{
			AccountId = *Found;
		}
	}
	return AccountId;
}

EOS_ProductUserId FOnlineAccountIdRegistryEOS::GetProductUserId(const FAccountId& AccountId) const
{
	return GetAccountIdData(AccountId).ProductUserId;
}

FAccountId FOnlineAccountIdRegistryEOS::FindAccountId(const EOS_EpicAccountId EpicAccountId) const
{
	FAccountId AccountId;
	if (EOS_EpicAccountId_IsValid(EpicAccountId) == EOS_TRUE)
	{
		const UE::TReadScopeLock ReadLock(Lock);
		if (const FAccountId* Found = EpicAccountIdToAccountId.Find(EpicAccountId))
		{
			AccountId = *Found;
		}
	}
	return AccountId;
}

FOnlineAccountIdDataEOS FOnlineAccountIdRegistryEOS::GetAccountIdData(const FAccountId& AccountId) const
{
	if (ValidateOnlineId(AccountId))
	{
		UE::TReadScopeLock ScopeLock(Lock);
		return AccountIdData[AccountId.GetHandle() - 1];
	}
	return FOnlineAccountIdDataEOS();
}

FOnlineAccountIdRegistryEOS& FOnlineAccountIdRegistryEOS::Get()
{
	static FOnlineAccountIdRegistryEOS Instance;
	return Instance;
}

FAccountId FOnlineAccountIdRegistryEOS::FindOrAddAccountId(const EOS_EpicAccountId InEpicAccountId, const EOS_ProductUserId InProductUserId)
{
	FAccountId Result;
	const bool bInEpicAccountIdValid = EOS_EpicAccountId_IsValid(InEpicAccountId) == EOS_TRUE;
	const bool bInProductUserIdValid = EOS_ProductUserId_IsValid(InProductUserId) == EOS_TRUE;
	if (ensure(bInEpicAccountIdValid || bInProductUserIdValid))
	{
		bool bUpdateEpicAccountId = false;
		bool bUpdateProductUserId = false;

		auto FindExisting = [this, InEpicAccountId, InProductUserId, bInEpicAccountIdValid, bInProductUserIdValid, &bUpdateEpicAccountId, &bUpdateProductUserId]()
		{
			FAccountId Result;
			if (const FAccountId* FoundEas = bInEpicAccountIdValid ? EpicAccountIdToAccountId.Find(InEpicAccountId) : nullptr)
			{
				Result = *FoundEas;
			}
			else if (const FAccountId* FoundProd = bInProductUserIdValid ? ProductUserIdToAccountId.Find(InProductUserId) : nullptr)
			{
				Result = *FoundProd;
			}

			if (Result.IsValid())
			{
				const EOS_EpicAccountId FoundEpicAccountId = AccountIdData[Result.GetHandle() - 1].EpicAccountId;
				const EOS_ProductUserId FoundProductUserId = AccountIdData[Result.GetHandle() - 1].ProductUserId;

				// Check that the found EAS/EOS ids are either unset, or match the input. If a valid input is passed for a currently unset field, this is an update,
				// which we will track here and complete later under a write lock.
				check(!FoundEpicAccountId || InEpicAccountId == FoundEpicAccountId);
				check(!FoundProductUserId || InProductUserId == FoundProductUserId);
				bUpdateEpicAccountId = !FoundEpicAccountId && bInEpicAccountIdValid;
				bUpdateProductUserId = !FoundProductUserId && bInProductUserIdValid;
			}

			return Result;
		};

		{
			// First take read lock and look for existing elements
			const UE::TReadScopeLock ReadLock(Lock);
			Result = FindExisting();
		}

		if(!Result.IsValid())
		{
			// Double-checked locking. If we didn't find an element, we take the write lock, and look again, in case another thread raced with us and added one.
			const UE::TWriteScopeLock WriteLock(Lock);
			Result = FindExisting();

			if (!Result.IsValid())
			{
				// We still didn't find one, so now we can add one.
				FOnlineAccountIdDataEOS& NewAccountIdData = AccountIdData.Emplace_GetRef();
				NewAccountIdData.EpicAccountId = InEpicAccountId;
				NewAccountIdData.ProductUserId = InProductUserId;

				Result = FAccountId(EOnlineServices::Epic, AccountIdData.Num());

				if (bInEpicAccountIdValid)
				{
					EpicAccountIdToAccountId.Emplace(InEpicAccountId, Result);
				}
				if (bInProductUserIdValid)
				{
					ProductUserIdToAccountId.Emplace(InProductUserId, Result);
				}
			}
		}

		check(Result.IsValid());
		if (bUpdateEpicAccountId || bUpdateProductUserId)
		{
			// Finally, update any previously unset fields for which we now have a valid value.
			const UE::TWriteScopeLock WriteLock(Lock);
			FOnlineAccountIdDataEOS& AccountIdDataToUpdate = AccountIdData[Result.GetHandle() -1];
			if (bUpdateEpicAccountId)
			{
				AccountIdDataToUpdate.EpicAccountId = InEpicAccountId;
				EpicAccountIdToAccountId.Emplace(InEpicAccountId, Result);
			}
			if (bUpdateProductUserId)
			{
				AccountIdDataToUpdate.ProductUserId = InProductUserId;
				ProductUserIdToAccountId.Emplace(InProductUserId, Result);
			}
		}
	}
	
	return Result;
}

EOS_EpicAccountId GetEpicAccountId(const FAccountId& AccountId)
{
	EOnlineServices ServicesType = AccountId.GetOnlineServicesType();

	if (ServicesType == EOnlineServices::EpicAccount)
	{
		IOnlineAccountIdRegistry* Registry = FOnlineIdRegistryRegistry::Get().GetAccountIdRegistry(ServicesType);
		check(Registry);

		IOnlineAccountIdRegistryEpicAccount* EpicAccountRegistry = static_cast<IOnlineAccountIdRegistryEpicAccount*>(Registry);
		check(EpicAccountRegistry);

		return EpicAccountRegistry->GetEpicAccountId(AccountId);
	}
	else if (ServicesType == EOnlineServices::Epic)
	{
		return FOnlineAccountIdRegistryEOS::Get().GetAccountIdData(AccountId).EpicAccountId;
	}

	return nullptr;
}

EOS_EpicAccountId GetEpicAccountIdChecked(const FAccountId& AccountId)
{
	EOS_EpicAccountId EpicAccountId = GetEpicAccountId(AccountId);
	check(EOS_EpicAccountId_IsValid(EpicAccountId) == EOS_TRUE);
	return EpicAccountId;
}

FAccountId FindAccountId(const EOS_EpicAccountId EpicAccountId)
{
	return FindAccountId(EOnlineServices::Epic, EpicAccountId);
}

FAccountId FindAccountId(EOnlineServices Services, const EOS_EpicAccountId EpicAccountId)
{
	if (Services == EOnlineServices::EpicAccount)
	{
		IOnlineAccountIdRegistry* Registry = FOnlineIdRegistryRegistry::Get().GetAccountIdRegistry(Services);
		check(Registry);

		IOnlineAccountIdRegistryEpicAccount* EpicAccountRegistry = static_cast<IOnlineAccountIdRegistryEpicAccount*>(Registry);
		check(EpicAccountRegistry);

		return EpicAccountRegistry->GetAccountId(EpicAccountId);
	}
	else if (Services == EOnlineServices::Epic)
	{
		return FOnlineAccountIdRegistryEOS::Get().FindAccountId(EpicAccountId);
	}

	return FAccountId();	
}

FAccountId FindAccountIdChecked(const EOS_EpicAccountId EpicAccountId)
{
	return FindAccountIdChecked(EOnlineServices::Epic, EpicAccountId);
}

FAccountId FindAccountIdChecked(EOnlineServices Services, const EOS_EpicAccountId EpicAccountId)
{
	FAccountId Result = FindAccountId(Services, EpicAccountId);
	check(Result.IsValid());
	return Result;
}

FAccountId CreateAccountId(const EOS_EpicAccountId EpicAccountId, const EOS_ProductUserId ProductUserId)
{
	return FOnlineAccountIdRegistryEOS::Get().FindOrAddAccountId(EpicAccountId, ProductUserId);
}

/* UE::Online */ }
