// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineIdEOSGS.h"
#include "Online/AccountIdEOS.h"
#include "Misc/TransactionallySafeRWLock.h"

struct EOS_EpicAccountIdDetails;
struct EOS_ProductUserIdDetails;

class FAccountIdReplicationTest;

namespace UE::Online {

class FOnlineAccountIdDataEOS
{
public:
	EOS_EpicAccountId EpicAccountId = nullptr;
	EOS_ProductUserId ProductUserId = nullptr;
};

/**
 * Account id registry specifically for EOS id's which are an EOS_EpicAccountId/EOS_ProductUserId pair.
 */
class FOnlineAccountIdRegistryEOS : public IOnlineAccountIdRegistryEOSGS
{
public:
	virtual ~FOnlineAccountIdRegistryEOS() = default;

	// Begin IOnlineAccountIdRegistry
	virtual FString ToString(const FAccountId& AccountId) const override;
	virtual FString ToLogString(const FAccountId& AccountId) const override;
	virtual TArray<uint8> ToReplicationData(const FAccountId& AccountId) const override;
	virtual FAccountId FromReplicationData(const TArray<uint8>& ReplicationString) override;
	virtual FAccountId FromStringData(const FString& StringData) override;
	// End IOnlineAccountIdRegistry

	// Begin IOnlineAccountRegistryEOSGS
	virtual FAccountId FindOrAddAccountId(const EOS_ProductUserId ProductUserId) override;
	virtual FAccountId FindAccountId(const EOS_ProductUserId ProductUserId) const override;
	virtual EOS_ProductUserId GetProductUserId(const FAccountId& AccountId) const override;
	// End IOnlineAccountRegistryEOSGS

	FAccountId FindAccountId(const EOS_EpicAccountId EpicAccountId) const;
	FAccountId FindOrAddAccountId(const EOS_EpicAccountId EpicAccountId, const EOS_ProductUserId ProductUserId);

	// Return copies as it is not thread safe to return pointers/references to array elements, in case the array is grown+relocated on another thread.
	FOnlineAccountIdDataEOS GetAccountIdData(const FAccountId& AccountId) const;

	static FOnlineAccountIdRegistryEOS& Get();

	template<typename IdType>
	static inline bool ValidateOnlineId(const TOnlineId<IdType> OnlineId)
	{
		return OnlineId.IsValid() && (OnlineId.GetOnlineServicesType() == EOnlineServices::Epic);
	}

private:
	friend class FAuthEOS;
	friend class FOnlineServicesEOSModule;
	friend FAccountIdReplicationTest;

	mutable FTransactionallySafeRWLock Lock;

	TArray<FOnlineAccountIdDataEOS> AccountIdData; // Actual container for the info, indexed by the handle

	TMap<EOS_EpicAccountId, FAccountId> EpicAccountIdToAccountId; // Map of EOS_EpicAccountId to the associated handle.
	TMap<EOS_ProductUserId, FAccountId> ProductUserIdToAccountId; // Map of EOS_ProductUserId to the associated handle.
};

} /* namespace UE::Online */
