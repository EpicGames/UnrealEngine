// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEOS.h"

#include "Online/AuthEOS.h"
#include "Online/EpicAccountIdResolverEOS.h"
#include "Online/EpicProductUserIdResolverEOS.h"
#include "Online/ExternalUIEOS.h"
#include "Online/SocialEOS.h"
#include "Online/PresenceEOS.h"
#include "Online/UserInfoEOS.h"
#include "Online/CommerceEOS.h"

namespace UE::Online {

FOnlineServicesEOS::FOnlineServicesEOS(FName InInstanceName, FName InInstanceConfigName)
	: Super(InInstanceName, InInstanceConfigName)
{
}

void FOnlineServicesEOS::RegisterComponents()
{
	Components.Register<FAuthEOS>(*this);
	Components.Register<FEpicAccountIdResolverEOS>(*this);
	Components.Register<FEpicProductUserIdResolverEOS>(*this);
	Components.Register<FExternalUIEOS>(*this);
	Components.Register<FSocialEOS>(*this);
	Components.Register<FPresenceEOS>(*this);
	Components.Register<FUserInfoEOS>(*this);
	Components.Register<FCommerceEOS>(*this);

	Super::RegisterComponents();
}

/* UE::Online */ }
