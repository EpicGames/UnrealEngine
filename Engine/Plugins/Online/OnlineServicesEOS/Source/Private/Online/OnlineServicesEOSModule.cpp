// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEOSModule.h"

#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "Online/OnlineExecHandler.h"
#include "Online/OnlineIdEOS.h"
#include "Online/OnlineServicesEOS.h"
#include "Online/OnlineServicesEOSGSModule.h"
#include "Online/OnlineServicesRegistry.h"

namespace UE::Online
{

class FOnlineServicesFactoryEOS : public IOnlineServicesFactory
{
public:
	virtual ~FOnlineServicesFactoryEOS() {}
	virtual TSharedPtr<IOnlineServices> Create(FName InInstanceName, FName InInstanceConfigName) override
	{
		TSharedPtr<FOnlineServicesEOS> Result = MakeShared<FOnlineServicesEOS>(InInstanceName, InInstanceConfigName);
		if (!Result->PreInit())
		{
			Result = nullptr;
		}
		return Result;
	}
};

int FOnlineServicesEOSModule::GetRegistryPriority()
{
	return FOnlineServicesEOSGSModule::GetRegistryPriority() + 1;
}

void FOnlineServicesEOSModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("OnlineServicesInterface"));

	if (TEMP_ShouldUseEpicModularPlugins())
	{
		UE_LOGF(LogOnlineServices, Verbose, "TEMP_ShouldUseEpicModularPlugins() = true, skipping FOnlineServicesEOSModule startup.");
		return;
	}

	FModuleManager::Get().LoadModuleChecked(TEXT("OnlineServicesEOSGS"));

	FOnlineServicesRegistry::Get().RegisterServicesFactory(EOnlineServices::Epic, MakeUnique<FOnlineServicesFactoryEOS>(), GetRegistryPriority());
	FOnlineIdRegistryRegistry::Get().RegisterAccountIdRegistry(EOnlineServices::Epic, &FOnlineAccountIdRegistryEOS::Get(), GetRegistryPriority());
}

void FOnlineServicesEOSModule::ShutdownModule()
{
	FOnlineServicesRegistry::Get().UnregisterServicesFactory(EOnlineServices::Epic, GetRegistryPriority());
	FOnlineIdRegistryRegistry::Get().UnregisterAccountIdRegistry(EOnlineServices::Epic, GetRegistryPriority());
}

/* UE::Online */ }

IMPLEMENT_MODULE(UE::Online::FOnlineServicesEOSModule, OnlineServicesEOS);
