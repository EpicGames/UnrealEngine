// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SocialEOS.h"

#include "IEOSSDKManager.h"
#include "Online/AuthEOS.h"
#include "Online/EpicAccountIdResolver.h"
#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineIdEOS.h"
#include "Online/OnlineServicesEOS.h"

#include "eos_friends.h"
#include "eos_ui.h"

namespace UE::Online {

ERelationship EOSEFriendStatusToERelationship(EOS_EFriendsStatus Status)
{
	switch (Status)
	{
	case EOS_EFriendsStatus::EOS_FS_NotFriends:		return ERelationship::NotFriend;
	case EOS_EFriendsStatus::EOS_FS_InviteSent:		return ERelationship::InviteSent;
	case EOS_EFriendsStatus::EOS_FS_InviteReceived:	return ERelationship::InviteReceived;
	case EOS_EFriendsStatus::EOS_FS_Friends:		return ERelationship::Friend;
	}

	return ERelationship::NotFriend;
}

FSocialEOS::FSocialEOS(FOnlineServicesEpicCommon& InServices)
	: FSocialCommon(InServices)
{
}

void FSocialEOS::Initialize()
{
	FSocialCommon::Initialize();

	FriendsHandle = EOS_Platform_GetFriendsInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(FriendsHandle != nullptr);

	UIHandle = EOS_Platform_GetUIInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());

	OnFriendsUpdate = EOS_RegisterComponentEventHandler(
		this,
		FriendsHandle,
		1,
		&EOS_Friends_AddNotifyFriendsUpdate,
		&EOS_Friends_RemoveNotifyFriendsUpdate,
		&FSocialEOS::HandleFriendsUpdate);
	UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_ADDNOTIFYFRIENDSUPDATE_API_LATEST, 1);
}

void FSocialEOS::PreShutdown()
{
	Super::PreShutdown();

	OnFriendsUpdate = nullptr;
}

TOnlineAsyncOpHandle<FQueryFriends> FSocialEOS::QueryFriends(FQueryFriends::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryFriends> Op = GetJoinableOp<FQueryFriends>(MoveTemp(InParams));
	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryFriends>& Op)
	{
		const FQueryFriends::Params& Params = Op.GetParams();

		if (!Params.LocalAccountId.IsValid())
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FQueryFriends>& Op, TPromise<const EOS_Friends_QueryFriendsCallbackInfo*>&& Promise)
	{
		EOS_Friends_QueryFriendsOptions QueryFriendsOptions = { };
		QueryFriendsOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_QUERYFRIENDS_API_LATEST, 1);
		QueryFriendsOptions.LocalUserId = GetEpicAccountIdChecked(Op.GetParams().LocalAccountId);

		EOS_Async(EOS_Friends_QueryFriends, FriendsHandle, QueryFriendsOptions, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FQueryFriends>& Op, const EOS_Friends_QueryFriendsCallbackInfo* Data)
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
			return TArray<EOS_EpicAccountId>();
		}

		EOS_Friends_GetFriendsCountOptions GetFriendsCountOptions = {
			EOS_FRIENDS_GETFRIENDSCOUNT_API_LATEST,
			Data->LocalUserId
		};

		int32_t NumFriends = EOS_Friends_GetFriendsCount(FriendsHandle, &GetFriendsCountOptions);
		TArray<EOS_EpicAccountId> CurrentFriends;
		CurrentFriends.Reserve(NumFriends);
		for (int32_t FriendIndex = 0; FriendIndex < NumFriends; ++FriendIndex)
		{
			EOS_Friends_GetFriendAtIndexOptions GetFriendAtIndexOptions = {
				EOS_FRIENDS_GETFRIENDATINDEX_API_LATEST,
				Data->LocalUserId,
				FriendIndex
			};

			const EOS_EpicAccountId FriendEasId = EOS_Friends_GetFriendAtIndex(FriendsHandle, &GetFriendAtIndexOptions);
			check(EOS_EpicAccountId_IsValid(FriendEasId));
			CurrentFriends.Emplace(FriendEasId);
		}
		return CurrentFriends;
	})
	.Then(Services.Get<IEpicAccountIdResolver>()->ResolveEpicAccountIdsFn())
	.Then([this](TOnlineAsyncOp<FQueryFriends>& Op, const TArray<FAccountId>& CurrentFriendIds)
	{
		const FAccountId LocalUserId = Op.GetParams().LocalAccountId;
		const EOS_EpicAccountId LocalUserEasId = GetEpicAccountIdChecked(LocalUserId);

		TMap<FAccountId, TSharedRef<FFriend>>& FriendsList = FriendsLists.FindOrAdd(LocalUserId);
				
		for (const FAccountId& FriendId : CurrentFriendIds)
		{
			const EOS_EpicAccountId FriendEasId = GetEpicAccountIdChecked(FriendId);

			EOS_Friends_GetStatusOptions GetStatusOptions = {
				EOS_FRIENDS_GETSTATUS_API_LATEST,
				LocalUserEasId,
				FriendEasId
			};
			const EOS_EFriendsStatus EOSFriendStatus = EOS_Friends_GetStatus(FriendsHandle, &GetStatusOptions);
			const ERelationship Relationship = EOSEFriendStatusToERelationship(EOSFriendStatus);

			if (TSharedRef<FFriend>* FriendPtr = FriendsList.Find(FriendId))
			{
				if (Relationship != (*FriendPtr)->Relationship)
				{
					ERelationship OldRelationship = (*FriendPtr)->Relationship;
					(*FriendPtr)->Relationship = Relationship;

					BroadcastRelationshipUpdated(LocalUserId, FriendId, OldRelationship, Relationship);
				}
			}
			else
			{
				TSharedRef<FFriend> Friend = MakeShared<FFriend>();
				Friend->FriendId = FriendId;
				Friend->Relationship = Relationship;

				FriendsList.Emplace(FriendId, Friend);

				BroadcastRelationshipUpdated(LocalUserId, FriendId, ERelationship::NotFriend, Relationship);
			}
		}

		Op.SetResult(FQueryFriends::Result());
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetFriends> FSocialEOS::GetFriends(FGetFriends::Params&& Params)
{
	if (TMap<FAccountId, TSharedRef<FFriend>>* FriendsList = FriendsLists.Find(Params.LocalAccountId))
	{
		FGetFriends::Result Result;
		FriendsList->GenerateValueArray(Result.Friends);
		return TOnlineResult<FGetFriends>(MoveTemp(Result));
	}
	return TOnlineResult<FGetFriends>(Errors::InvalidState());
}

TOnlineAsyncOpHandle<FSendFriendInvite> FSocialEOS::SendFriendInvite(FSendFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FSendFriendInvite> Op = GetJoinableOp<FSendFriendInvite>(MoveTemp(Params));

	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FSendFriendInvite>& Op)
	{
		const FSendFriendInvite::Params& Params = Op.GetParams();

		if (!Params.LocalAccountId.IsValid() || !Params.TargetAccountId.IsValid() || !EOS_EpicAccountId_IsValid(GetEpicAccountIdChecked(Params.TargetAccountId)))
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FSendFriendInvite>& Op, TPromise<const EOS_Friends_SendInviteCallbackInfo*>&& Promise)
	{
		const FSendFriendInvite::Params& Params = Op.GetParams();

		EOS_Friends_SendInviteOptions SendInviteOptions = {};
		SendInviteOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_SENDINVITE_API_LATEST, 1);
		SendInviteOptions.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
		SendInviteOptions.TargetUserId = GetEpicAccountIdChecked(Params.TargetAccountId);

		EOS_Async(EOS_Friends_SendInvite, FriendsHandle, SendInviteOptions, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FSendFriendInvite>& Op, const EOS_Friends_SendInviteCallbackInfo* Data)
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
			return;
		}

		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FAcceptFriendInvite> FSocialEOS::AcceptFriendInvite(FAcceptFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FAcceptFriendInvite> Op = GetJoinableOp<FAcceptFriendInvite>(MoveTemp(Params));

	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FAcceptFriendInvite>& Op)
	{
		const FAcceptFriendInvite::Params& Params = Op.GetParams();

		if (!Params.LocalAccountId.IsValid() || !Params.TargetAccountId.IsValid() || !EOS_EpicAccountId_IsValid(GetEpicAccountIdChecked(Params.TargetAccountId)))
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FAcceptFriendInvite>& Op, TPromise<const EOS_Friends_AcceptInviteCallbackInfo*>&& Promise)
	{
		const FAcceptFriendInvite::Params& Params = Op.GetParams();

		EOS_Friends_AcceptInviteOptions AcceptInviteOptions = {};
		AcceptInviteOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_ACCEPTINVITE_API_LATEST, 1);
		AcceptInviteOptions.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
		AcceptInviteOptions.TargetUserId = GetEpicAccountIdChecked(Params.TargetAccountId);

		EOS_Async(EOS_Friends_AcceptInvite, FriendsHandle, AcceptInviteOptions, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FAcceptFriendInvite>& Op, const EOS_Friends_AcceptInviteCallbackInfo* Data)
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
			return;
		}

		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FRejectFriendInvite> FSocialEOS::RejectFriendInvite(FRejectFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FRejectFriendInvite> Op = GetJoinableOp<FRejectFriendInvite>(MoveTemp(Params));

	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FRejectFriendInvite>& Op)
	{
		const FRejectFriendInvite::Params& Params = Op.GetParams();

		if (!Params.LocalAccountId.IsValid() || !Params.TargetAccountId.IsValid() || !EOS_EpicAccountId_IsValid(GetEpicAccountIdChecked(Params.TargetAccountId)))
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FRejectFriendInvite>& Op, TPromise<const EOS_Friends_RejectInviteCallbackInfo*>&& Promise)
	{
		const FRejectFriendInvite::Params& Params = Op.GetParams();

		EOS_Friends_RejectInviteOptions RejectInviteOptions = {};
		RejectInviteOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_FRIENDS_REJECTINVITE_API_LATEST, 1);
		RejectInviteOptions.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
		RejectInviteOptions.TargetUserId = GetEpicAccountIdChecked(Params.TargetAccountId);

		EOS_Async(EOS_Friends_RejectInvite, FriendsHandle, RejectInviteOptions, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FRejectFriendInvite>& Op, const EOS_Friends_RejectInviteCallbackInfo* Data)
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
			return;
		}

		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FBlockUser> FSocialEOS::BlockUser(FBlockUser::Params&& Params)
{
	TOnlineAsyncOpRef<FBlockUser> Op = GetJoinableOp<FBlockUser>(MoveTemp(Params));

	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FBlockUser>& Op)
	{
		const FBlockUser::Params& Params = Op.GetParams();

		if (!UIHandle)
		{
			Op.SetError(Errors::MissingInterface());
			return;
		}

		if (!Params.LocalAccountId.IsValid() || !Params.TargetAccountId.IsValid() || !EOS_EpicAccountId_IsValid(GetEpicAccountIdChecked(Params.TargetAccountId)))
		{
			Op.SetError(Errors::InvalidParams());
			return;
		}

		if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op.SetError(Errors::NotLoggedIn());
			return;
		}
	})
	.Then([this](TOnlineAsyncOp<FBlockUser>& Op, TPromise<const EOS_UI_OnShowBlockPlayerCallbackInfo*>&& Promise)
	{
		const FBlockUser::Params& Params = Op.GetParams();

		EOS_UI_ShowBlockPlayerOptions ShowBlockPlayerOptions = {};
		ShowBlockPlayerOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_UI_SHOWBLOCKPLAYER_API_LATEST, 1);
		ShowBlockPlayerOptions.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
		ShowBlockPlayerOptions.TargetUserId = GetEpicAccountIdChecked(Params.TargetAccountId);

		EOS_Async(EOS_UI_ShowBlockPlayer, UIHandle, ShowBlockPlayerOptions, MoveTemp(Promise));
	})
	.Then([this](TOnlineAsyncOp<FBlockUser>& Op, const EOS_UI_OnShowBlockPlayerCallbackInfo* Data)
	{
		if (Data->ResultCode != EOS_EResult::EOS_Success)
		{
			Op.SetError(Errors::FromEOSResult(Data->ResultCode));
			return;
		}

		Op.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

void FSocialEOS::HandleFriendsUpdate(const EOS_Friends_OnFriendsUpdateInfo* Data)
{		
	// Note this local user may not exist if login has not been called... (more accurately if login has occured outside of this plugin)
	const FAccountId LocalAccountId = FindAccountId(Data->LocalUserId);
	if (!LocalAccountId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Log, "%s LocalUserId=%ls not found.", __FUNCTION__, *LexToString(Data->LocalUserId));
		return;
	}

	Services.Get<IEpicAccountIdResolver>()->ResolveAccountId(LocalAccountId, Data->TargetUserId)
	.Next([this, LocalAccountId, PreviousStatus = Data->PreviousStatus, CurrentStatus = Data->CurrentStatus](const FAccountId& FriendAccountId)
	{
		const EOS_EpicAccountId LocalUserEasId = GetEpicAccountIdChecked(LocalAccountId);

		TMap<FAccountId, TSharedRef<FFriend>>& FriendsList = FriendsLists.FindOrAdd(LocalAccountId);

		if (CurrentStatus == EOS_EFriendsStatus::EOS_FS_NotFriends)
		{
			FriendsList.Remove(FriendAccountId);
		}
		else
		{
			// Add/update friend with appropriate status
			if (TSharedRef<FFriend>* ExistingFriend = FriendsList.Find(FriendAccountId))
			{
				(*ExistingFriend)->Relationship = EOSEFriendStatusToERelationship(CurrentStatus);
			}
			else
			{
				// Create new entry
				TSharedRef<FFriend> Friend = MakeShared<FFriend>();
				Friend->FriendId = FriendAccountId;
				Friend->Relationship = EOSEFriendStatusToERelationship(CurrentStatus);
				FriendsList.Emplace(FriendAccountId, MoveTemp(Friend));
			}
		}

		BroadcastRelationshipUpdated(LocalAccountId, FriendAccountId, EOSEFriendStatusToERelationship(PreviousStatus), EOSEFriendStatusToERelationship(CurrentStatus));
	});
}

FAccountId FSocialEOS::FindAccountId(const EOS_EpicAccountId EpicAccountId)
{
	return UE::Online::FindAccountId(Services.GetServicesProvider(), EpicAccountId);
}

/* UE::Online */ }
