// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "IEOSSDKManager.h"
#include "Online/OnlineIdEOS.h"
#include <eos_base.h>

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAccountIdReplicationTest, 
	"System.Engine.Online.EosAccountIdReplicationTest", 
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FAccountIdReplicationTest::RunTest(const FString& Parameters)
{
	IEOSSDKManager* SDKManager = IEOSSDKManager::Get();
	if(ensure(SDKManager) && ensure(SDKManager->IsInitialized()))
	{
		using namespace UE::Online;

		const EOS_EpicAccountId EasId = EOS_EpicAccountId_FromString("d03352b601a44af7b02ed1faab668e0f");
		check(EOS_EpicAccountId_IsValid(EasId));
		const EOS_ProductUserId EosId = EOS_ProductUserId_FromString("31a0b16a1f0743978458e3733bb3be14");
		check(EOS_ProductUserId_IsValid(EosId));

		{
			FOnlineAccountIdRegistryEOS Registry;
			const FAccountId AccountId = Registry.FindOrAddAccountId(EasId, EosId);
			TArray<uint8> RepData = Registry.ToReplicationData(AccountId);
			const FAccountId AccountId2 = Registry.FromReplicationData(RepData);
			UTEST_EQUAL(TEXT(""), AccountId, AccountId2);
		}

		{
			FOnlineAccountIdRegistryEOS Registry;
			const FAccountId AccountId = Registry.FindOrAddAccountId(EasId, nullptr);
			TArray<uint8> RepData = Registry.ToReplicationData(AccountId);
			const FAccountId AccountId2 = Registry.FromReplicationData(RepData);
			UTEST_EQUAL(TEXT(""), AccountId, AccountId2);
		}

		{
			FOnlineAccountIdRegistryEOS Registry;
			const FAccountId AccountId = Registry.FindOrAddAccountId(nullptr, EosId);
			TArray<uint8> RepData = Registry.ToReplicationData(AccountId);
			const FAccountId AccountId2 = Registry.FromReplicationData(RepData);
			UTEST_EQUAL(TEXT(""), AccountId, AccountId2);
		}
	}

	return true;
}
