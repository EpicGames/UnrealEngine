// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserInfoEOS.h"

#include "EOSShared.h"
#include "IEOSSDKManager.h"
#include "Online/OnlineIdEOS.h"
#include "Online/OnlineServicesEOS.h"
#include "Online/AuthEOS.h"
#include "Online/OnlineErrorEpicCommon.h"

#include "eos_userinfo.h"

namespace UE::Online {

FUserInfoEOS::FUserInfoEOS(FOnlineServicesEpicCommon& InServices)
	: FUserInfoCommon(InServices)
{
}

void FUserInfoEOS::Initialize()
{
	Super::Initialize();

	UserInfoHandle = EOS_Platform_GetUserInfoInterface(*GetServices<FOnlineServicesEpicCommon>().GetEOSPlatformHandle());
	check(UserInfoHandle != nullptr);
}

TOnlineAsyncOpHandle<FQueryUserInfo> FUserInfoEOS::QueryUserInfo(FQueryUserInfo::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryUserInfo> Op = GetJoinableOp<FQueryUserInfo>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FQueryUserInfo::Params& Params = Op->GetParams();

		if (Params.AccountIds.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		const bool bIsLoggedInConnect = EOS_ProductUserId_IsValid(GetProductUserId(Params.LocalAccountId)) == EOS_TRUE;

		for (const FAccountId TargetAccountId : Params.AccountIds)
		{
			Op->Then([this, TargetAccountId](TOnlineAsyncOp<FQueryUserInfo>& Op, TPromise<const EOS_UserInfo_QueryUserInfoCallbackInfo*>&& Promise)
			{
				const FQueryUserInfo::Params& Params = Op.GetParams();

				if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
				{
					Op.SetError(Errors::NotLoggedIn());
					Promise.EmplaceValue();
					return;
				}

				const EOS_EpicAccountId TargetUserEasId = GetEpicAccountId(TargetAccountId);
				if (!EOS_EpicAccountId_IsValid(TargetUserEasId))
				{
					Op.SetError(Errors::InvalidParams());
					Promise.EmplaceValue();
					return;
				}

				EOS_UserInfo_QueryUserInfoOptions QueryUserInfoOptions = {};
				QueryUserInfoOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_QUERYUSERINFO_API_LATEST, 1);
				QueryUserInfoOptions.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
				QueryUserInfoOptions.TargetUserId = TargetUserEasId;

				EOS_Async(EOS_UserInfo_QueryUserInfo, UserInfoHandle, QueryUserInfoOptions, MoveTemp(Promise));
			})
			.Then([this](TOnlineAsyncOp<FQueryUserInfo>& Op, const EOS_UserInfo_QueryUserInfoCallbackInfo* CallbackInfo) mutable
			{
				if (CallbackInfo->ResultCode != EOS_EResult::EOS_Success)
				{
					UE_LOGF(LogOnlineServices, Warning, "EOS_UserInfo_QueryUserInfo failed with result=[%ls]", *LexToString(CallbackInfo->ResultCode));
					Op.SetError(Errors::FromEOSResult(CallbackInfo->ResultCode));
				}
			});

			if (bIsLoggedInConnect)
			{
				Op->Then([this, TargetAccountId](TOnlineAsyncOp<FQueryUserInfo>& Op, TPromise<const EOS_Connect_QueryExternalAccountMappingsCallbackInfo*> && Promise)
				{
					const FQueryUserInfo::Params& Params = Op.GetParams();
					const EOS_EpicAccountId TargetUserEasId = GetEpicAccountIdChecked(TargetAccountId);
					const FString TargetUserEasIdStr = LexToString(TargetUserEasId);
					const auto TargetUserEasIdUtf8StringCast = StringCast<UTF8CHAR>(*TargetUserEasIdStr);
					const char* TargetUserEasIdUtf8Str = (const char*)TargetUserEasIdUtf8StringCast.Get();
					EOS_Connect_QueryExternalAccountMappingsOptions Options;
					Options.ApiVersion = 1;
					UE_EOS_CHECK_API_MISMATCH(EOS_CONNECT_QUERYEXTERNALACCOUNTMAPPINGS_API_LATEST, 1);
					Options.LocalUserId = GetProductUserIdChecked(Params.LocalAccountId);
					Options.AccountIdType = EOS_EExternalAccountType::EOS_EAT_EPIC;
					Options.ExternalAccountIds = &TargetUserEasIdUtf8Str;
					Options.ExternalAccountIdCount = 1;
					EOS_HConnect ConnectHandle = EOS_Platform_GetConnectInterface(*GetServices<FOnlineServicesEOS>().GetEOSPlatformHandle());

					GetServices<FOnlineServicesEpicCommon>().AddEOSSDKFastTick(Op);

					EOS_Async(EOS_Connect_QueryExternalAccountMappings, ConnectHandle, Options, MoveTemp(Promise));
				})
				.Then([this](TOnlineAsyncOp<FQueryUserInfo>& Op, const EOS_Connect_QueryExternalAccountMappingsCallbackInfo* CallbackInfo) mutable
				{
					GetServices<FOnlineServicesEpicCommon>().RemoveEOSSDKFastTick(Op);

					if (CallbackInfo->ResultCode != EOS_EResult::EOS_Success)
					{
						UE_LOGF(LogOnlineServices, Warning, "EOS_Connect_QueryExternalAccountMappings failed with result=[%ls]", *LexToString(CallbackInfo->ResultCode));
						Op.SetError(Errors::FromEOSResult(CallbackInfo->ResultCode));
					}
				});
			}
		}

		Op->Then([](TOnlineAsyncOp<FQueryUserInfo>& Op)
			{
				Op.SetResult({});
			});

		Op->Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FGetUserInfo> FUserInfoEOS::GetUserInfo(FGetUserInfo::Params&& Params)
{
	if (!Services.Get<IAuth>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetUserInfo>(Errors::NotLoggedIn());
	}

	const EOS_EpicAccountId TargetUserEasId = GetEpicAccountId(Params.AccountId);
	if (!EOS_EpicAccountId_IsValid(TargetUserEasId))
	{
		return TOnlineResult<FGetUserInfo>(Errors::InvalidParams());
	}

	EOS_UserInfo_BestDisplayName* EosBestDisplayName = nullptr;
	ON_SCOPE_EXIT
	{
		if (EosBestDisplayName)
		{
			EOS_UserInfo_BestDisplayName_Release(EosBestDisplayName);
		}
	};

	auto CopyBestDisplayNameWithPlatform = [this, &Params, TargetUserEasId](EOS_UserInfo_BestDisplayName** EosBestDisplayName)
	{
		EOS_UserInfo_CopyBestDisplayNameWithPlatformOptions WithPlatformOptions = {};
		WithPlatformOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAMEWITHPLATFORM_API_LATEST, 1);
		WithPlatformOptions.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
		WithPlatformOptions.TargetUserId = TargetUserEasId;

		EOS_OnlinePlatformType OnlinePlatformType = EOS_OPT_Epic;
		if (Params.DisplayNamePlatform.IsSet())
		{
			EOS_OnlinePlatformType NewOnlinePlatformType = EOnlinePlatformType_To_EOS_OnlinePlatformType(Params.DisplayNamePlatform.GetValue());
			if (NewOnlinePlatformType != EOS_OPT_Unknown)
			{
				OnlinePlatformType = NewOnlinePlatformType;
			}
			else
			{
				UE_LOGF(LogOnlineServices, VeryVerbose, "FGetUserInfo::Params::DisplayNamePlatform platform type unknown. Epic will be used as default.");
			}
		}
		
		WithPlatformOptions.TargetPlatformType = OnlinePlatformType;

		return EOS_UserInfo_CopyBestDisplayNameWithPlatform(UserInfoHandle, &WithPlatformOptions, EosBestDisplayName);
	};

	EOS_EResult EosResult;
	if (Params.DisplayNamePlatform.IsSet())
	{
		EosResult = CopyBestDisplayNameWithPlatform(&EosBestDisplayName);
	}
	else
	{
		EOS_UserInfo_CopyBestDisplayNameOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_USERINFO_COPYBESTDISPLAYNAME_API_LATEST, 1);
		Options.LocalUserId = GetEpicAccountIdChecked(Params.LocalAccountId);
		Options.TargetUserId = TargetUserEasId;

		EosResult = EOS_UserInfo_CopyBestDisplayName(UserInfoHandle, &Options, &EosBestDisplayName);

		if (EosResult == EOS_EResult::EOS_UserInfo_BestDisplayNameIndeterminate)
		{
			EosResult = CopyBestDisplayNameWithPlatform(&EosBestDisplayName);
		}
	}

	if(EosResult != EOS_EResult::EOS_Success)
	{
		UE_LOGF(LogOnlineServices, Warning, "EOS_UserInfo_CopyBestDisplayName failed with result=[%ls]", *LexToString(EosResult));
		return TOnlineResult<FGetUserInfo>(Errors::FromEOSResult(EosResult));
	}

	TSharedRef<FUserInfo> UserInfo = MakeShared<FUserInfo>();
	UserInfo->AccountId = Params.AccountId;
	UserInfo->DisplayName = GetBestDisplayNameStr(*EosBestDisplayName);

	return TOnlineResult<FGetUserInfo>({UserInfo});
}

/* UE::Online */ }
