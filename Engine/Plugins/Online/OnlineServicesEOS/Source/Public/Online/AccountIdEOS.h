// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"

typedef struct EOS_EpicAccountIdDetails* EOS_EpicAccountId;
typedef struct EOS_ProductUserIdDetails* EOS_ProductUserId;

namespace UE::Online
{
	class IOnlineAccountIdRegistryEpicAccount : public IOnlineAccountIdRegistry
	{
	public:
		virtual FAccountId GetAccountId(EOS_EpicAccountId EpicAccountId) = 0;
		virtual EOS_EpicAccountId GetEpicAccountId(const FAccountId& AccountId) const = 0;
	};

ONLINESERVICESEOS_API EOS_EpicAccountId GetEpicAccountId(const FAccountId& AccountId);
ONLINESERVICESEOS_API EOS_EpicAccountId GetEpicAccountIdChecked(const FAccountId& AccountId);

UE_DEPRECATED(5.6, "This method is deprecated, please use the new version taking a EOnlineServices parameter")
ONLINESERVICESEOS_API FAccountId FindAccountId(const EOS_EpicAccountId EpicAccountId);
ONLINESERVICESEOS_API FAccountId FindAccountId(EOnlineServices Services, const EOS_EpicAccountId EpicAccountId);

UE_DEPRECATED(5.6, "This method is deprecated, please use the new version taking a EOnlineServices parameter")
ONLINESERVICESEOS_API FAccountId FindAccountIdChecked(const EOS_EpicAccountId EpicAccountId);
ONLINESERVICESEOS_API FAccountId FindAccountIdChecked(EOnlineServices Services, const EOS_EpicAccountId EpicAccountId);

ONLINESERVICESEOS_API FAccountId CreateAccountId(const EOS_EpicAccountId EpicAccountId, const EOS_ProductUserId ProductUserId);

} /* namespace UE::Online */
