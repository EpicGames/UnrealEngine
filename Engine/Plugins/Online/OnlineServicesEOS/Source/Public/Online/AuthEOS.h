// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/AuthEOSGS.h"

#include "eos_userinfo_types.h"

#define UE_API ONLINESERVICESEOS_API

namespace UE::Online {

class FOnlineServicesEOS;

namespace ELinkAccountTag
{
// An internal account is an account which has nothing external allowing the user to login on the epicgames.com website.
ONLINESERVICESEOS_API extern const FName InternalAccount;
}

class FAuthEOS : public FAuthEOSGS
{
public:
	using Super = FAuthEOSGS;

	UE_API FAuthEOS(FOnlineServicesCommon& InOwningSubsystem);
	virtual ~FAuthEOS() = default;

	// Begin IOnlineComponent
	UE_API virtual void Initialize() override;
	// End IOnlineComponent

	// Begin IAuth
	UE_API virtual TOnlineAsyncOpHandle<FAuthLogin> Login(FAuthLogin::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthLinkAccount> LinkAccount(FAuthLinkAccount::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params) override;
	UE_API virtual TOnlineResult<FAuthGetLinkAccountContinuationId> GetLinkAccountContinuationId(FAuthGetLinkAccountContinuationId::Params&& Params) const;
	// End IAuth

	UE_API TFuture<TArray<FAccountId>> ResolveAccountIds(const FAccountId& LocalAccountId, const TArray<EOS_EpicAccountId>& EpicAccountIds);
	UE_API TFuture<TArray<FAccountId>> ResolveAccountIds(const FAccountId& LocalAccountId, const TArray<EOS_ProductUserId>& ProductUserIds);

protected:
	UE_API void ProcessSuccessfulLogin(TOnlineAsyncOp<FAuthLogin>& InAsyncOp);
	UE_API void OnEASLoginStatusChanged(FAccountId LocalAccountId, ELoginStatus PreviousStatus, ELoginStatus CurrentStatus);

	UE_API FAccountId FindAccountId(const EOS_ProductUserId ProductUserId);
	UE_API FAccountId FindAccountId(const EOS_EpicAccountId EpicAccountId);

	static UE_API FAccountId CreateAccountId(const EOS_EpicAccountId EpicAccountId, const EOS_ProductUserId ProductUserId);

	EOS_HUserInfo UserInfoHandle = nullptr;
};

using FAuthEOSPtr = TSharedPtr<FAuthEOS>;

/* UE::Online */ }

#undef UE_API
