// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_ecom.h"

namespace UE::Online {

class FOnlineServicesEOS;

class FCommerceEOS : public FCommerceCommon
{
public:
	using Super = FCommerceCommon;
	using FCommerceCommon::FCommerceCommon;

	// IOnlineComponent
	ONLINESERVICESEOS_API virtual void PostInitialize() override;
	ONLINESERVICESEOS_API virtual void PreShutdown() override;

	// ICommerce
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceQueryOffers> QueryOffers(FCommerceQueryOffers::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceQueryOffersById> QueryOffersById(FCommerceQueryOffersById::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineResult<FCommerceGetOffers> GetOffers(FCommerceGetOffers::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineResult<FCommerceGetOffersById> GetOffersById(FCommerceGetOffersById::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceShowStoreUI> ShowStoreUI(FCommerceShowStoreUI::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceCheckout> Checkout(FCommerceCheckout::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceQueryEntitlements> QueryEntitlements(FCommerceQueryEntitlements::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineResult<FCommerceGetEntitlements> GetEntitlements(FCommerceGetEntitlements::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params) override;

private:
	void EOSOfferToOssOffer(FOffer& InOffer, EOS_Ecom_CatalogOffer* EosOffer);
	void EOSEntitlementToOssEntitlement(FEntitlement& OutEntitlement, EOS_Ecom_Entitlement* EosEntitlement);

	EOS_HEcom EcomHandle;
	TMap<FAccountId, TArray<FEntitlement>> CachedEntitlements;
	TMap<FAccountId, TArray<FOffer>> CachedOffers;
};

/* UE::Online */
}
