// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/EpicAccountIdResolver.h"
#include "Online/OnlineComponent.h"

namespace UE::Online {

class FEpicAccountIdResolverEOS : public TOnlineComponent<IEpicAccountIdResolver>
{
public:
	FEpicAccountIdResolverEOS(class FOnlineServicesEOS& InServices);

	// Begin IEpicAccountIdResolver
	virtual TFuture<FAccountId> ResolveAccountId(const FAccountId LocalAccountId, const EOS_EpicAccountId EpicAccountId) override;
	virtual TFuture<TArray<FAccountId>> ResolveAccountIds(const FAccountId LocalAccountId, const TArray<EOS_EpicAccountId>& EpicAccountIds) override;
	// End IEpicAccountIdResolver
};

/* UE::Online */ }
