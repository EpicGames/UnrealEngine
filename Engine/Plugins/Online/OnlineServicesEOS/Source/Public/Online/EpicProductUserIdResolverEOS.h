// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/EpicProductUserIdResolver.h"
#include "Online/OnlineComponent.h"

namespace UE::Online {

class FEpicProductUserIdResolverEOS : public TOnlineComponent<IEpicProductUserIdResolver>
{
public:
	FEpicProductUserIdResolverEOS(class FOnlineServicesEOS& InServices);

	// Begin IEpicProductUserIdResolver
	virtual TFuture<FAccountId> ResolveAccountId(const FAccountId LocalAccountId, const EOS_ProductUserId ProductUserId) override;
	virtual TFuture<TArray<FAccountId>> ResolveAccountIds(const FAccountId LocalAccountId, const TArray<EOS_ProductUserId>& ProductUserIds) override;
	// End IEpicProductUserIdResolver
};

/* UE::Online */ }
