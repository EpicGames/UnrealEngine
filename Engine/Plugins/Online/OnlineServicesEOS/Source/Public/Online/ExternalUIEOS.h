// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/ExternalUIEOSGS.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_ui_types.h"

namespace UE::Online {

struct FExternalUIEOSConfig
{
	/** Is ShowLoginUI enabled? */
	bool bShowLoginUIEnabled = true;
};

class FOnlineServicesEpicCommon;

class FExternalUIEOS : public FExternalUIEOSGS
{
public:
	using Super = FExternalUIEOSGS;

	ONLINESERVICESEOS_API FExternalUIEOS(FOnlineServicesEpicCommon& InOwningSubsystem);
	ONLINESERVICESEOS_API virtual ~FExternalUIEOS();

	ONLINESERVICESEOS_API virtual void Initialize() override;
	ONLINESERVICESEOS_API virtual void UpdateConfig() override;

	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FExternalUIShowLoginUI> ShowLoginUI(FExternalUIShowLoginUI::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params) override;

protected:
	/** Handle to EOS UI */
	EOS_HUI UIHandle;
	/** Config */
	FExternalUIEOSConfig Config;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FExternalUIEOSConfig)
	ONLINE_STRUCT_FIELD(FExternalUIEOSConfig, bShowLoginUIEnabled)
END_ONLINE_STRUCT_META()

/* Meta */ }

/* UE::Online */ }
