// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineServicesEOSGS.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_sdk.h"

#if WITH_ENGINE
class FSocketSubsystemEOS;
#endif

using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle>;

namespace UE::Online {

class FOnlineServicesEOS : public FOnlineServicesEOSGS
{
public:
	using Super = FOnlineServicesEOSGS;

	ONLINESERVICESEOS_API FOnlineServicesEOS(FName InInstanceName, FName InInstanceConfigName);
	virtual ~FOnlineServicesEOS() = default;

	ONLINESERVICESEOS_API virtual void RegisterComponents() override;
};

/* UE::Online */ }
