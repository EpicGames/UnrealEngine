// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "EOSSharedTypes.h"
#include "Online/OnlineServicesEOSGSTypes.h"
#include "Online/PresenceCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_presence_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;

class FPresenceEOS : public FPresenceCommon
{
public:
	using Super = FPresenceCommon;

	ONLINESERVICESEOS_API FPresenceEOS(FOnlineServicesEpicCommon& InServices);
	ONLINESERVICESEOS_API virtual ~FPresenceEOS();

	ONLINESERVICESEOS_API virtual void Initialize() override;
	ONLINESERVICESEOS_API virtual void PreShutdown() override;
	ONLINESERVICESEOS_API virtual void UpdateConfig() override;
	ONLINESERVICESEOS_API virtual void RegisterCommands() override;

	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FQueryPresence> QueryPresence(FQueryPresence::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineResult<FGetCachedPresence> GetCachedPresence(FGetCachedPresence::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresence(FUpdatePresence::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FPartialUpdatePresence> PartialUpdatePresence(FPartialUpdatePresence::Params&& Params) override;

protected:
	/** Get a user's presence, creating entries if missing */
	ONLINESERVICESEOS_API TSharedRef<FUserPresence> FindOrCreatePresence(FAccountId LocalAccountId, FAccountId PresenceAccountId);
	/** Update a user's presence from EOS's current value */
	ONLINESERVICESEOS_API void UpdateUserPresence(FAccountId LocalAccountId, FAccountId PresenceAccountId);
	/** Performs queued presence updates after a user's login completes */
	ONLINESERVICESEOS_API void HandleAuthLoginStatusChanged(const FAuthLoginStatusChanged& EventParameters);

	ONLINESERVICESEOS_API void HandlePresenceChanged(const EOS_Presence_PresenceChangedCallbackInfo* Data);

	ONLINESERVICESEOS_API FAccountId FindAccountId(const EOS_EpicAccountId EpicAccountId);

	/** Allow derived classes to modify the content of presence updates */
	ONLINESERVICESEOS_API virtual void ModifyPresenceUpdate(TSharedRef<FUserPresence>& Presence);
	/** Allow derived classes to modify the content of partial presence updates */
	ONLINESERVICESEOS_API virtual void ModifyPartialPresenceUpdate(FPartialUpdatePresence::Params::FMutations& Mutations);

	ONLINESERVICESEOS_API static EOnlinePlatformType IntegratedPlatform_To_OnlinePlatformType(const FString& IntegratedPlatformStr);
	ONLINESERVICESEOS_API virtual bool ReadOnlinePlatformType(const EOS_Presence_Info* PresenceInfo, EOnlinePlatformType& OutOnlinePlatformType);

	EOS_HPresence PresenceHandle = nullptr;

	/** Login status changed event handle */
	FOnlineEventDelegateHandle LoginStatusChangedHandle;

	TMap<EOS_EpicAccountId, TArray<EOS_EpicAccountId>> PendingPresenceUpdates;
	TMap<FAccountId, TMap<FAccountId, TSharedRef<FUserPresence>>> PresenceLists;
	
	FEOSEventRegistrationPtr OnPresenceChanged;

	double AsyncOpEnqueueDelay = 0.0;

	// Optimistic cache + Presence Update Retries

	/**
	 * Apply partial presence mutations to the local cache and broadcast OnPresenceUpdatedEvent if there was a change.
	 */
	void ApplyMutationsToCache(FAccountId LocalAccountId, const FPartialUpdatePresence::Params::FMutations& Mutations);

	/**
	 * Apply a full presence replacement to the local cache and broadcast OnPresenceUpdatedEvent if there was a change.
	 */
	void ApplyFullPresenceToCache(FAccountId LocalAccountId, const FUserPresence& NewPresence);

	/** Build a FMutations representing the delta between the EOS SDK's presence state and a desired presence. */
	FPartialUpdatePresence::Params::FMutations BuildPresenceDelta(const EOS_Presence_Info* EOSPresenceInfo, const FUserPresence& DesiredPresence);

	/** Mark a user's presence as needing a retry push to the EOS backend. */
	void SchedulePresenceRetry(FAccountId LocalAccountId);

	/** Execute pending presence retries by pushing the current cache state to EOS. */
	void StartPresenceRetry();

	/** Cancel the next scheduled retry for a user without resetting the retry counter. */
	void CancelPresenceRetry(FAccountId LocalAccountId);

	/** Fully remove pending retry state for a user. Called on success, max retries exceeded, or logout. */
	void ClearPresenceRetry(FAccountId LocalAccountId);

	/** Pending retry state for a user whose presence failed to push to EOS. */
	struct FPendingPresenceRetry
	{
		int32 RetryCount = 0;

		/** Whether this entry should fire on the next ticker tick. Set to false when a presence update method is called; RetryCount is preserved so any later failure in the same operation continues counting. */
		bool bPendingRetry = true;
	};

	TMap<FAccountId, FPendingPresenceRetry> PendingRetries;
	FTSTicker::FDelegateHandle RetryTickerHandle;
	double PresenceRetryDelaySec = 5.0;
	int32 MaxPresenceRetries = 3;
};

/* UE::Online */ }
