// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineServicesEOSGSTypes.h"
#include "Online/SocialCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_friends_types.h"
#include "eos_ui_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;
	
class FSocialEOS : public FSocialCommon
{
public:
	using Super = FSocialCommon;

	ONLINESERVICESEOS_API FSocialEOS(FOnlineServicesEpicCommon& InServices);

	ONLINESERVICESEOS_API virtual void Initialize() override;
	ONLINESERVICESEOS_API virtual void PreShutdown() override;

	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FQueryFriends> QueryFriends(FQueryFriends::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineResult<FGetFriends> GetFriends(FGetFriends::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FSendFriendInvite> SendFriendInvite(FSendFriendInvite::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FAcceptFriendInvite> AcceptFriendInvite(FAcceptFriendInvite::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FRejectFriendInvite> RejectFriendInvite(FRejectFriendInvite::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FBlockUser> BlockUser(FBlockUser::Params&& Params) override;

protected:
	ONLINESERVICESEOS_API void HandleFriendsUpdate(const EOS_Friends_OnFriendsUpdateInfo* Data);

	ONLINESERVICESEOS_API FAccountId FindAccountId(const EOS_EpicAccountId EpicAccountId);

	TMap<FAccountId, TMap<FAccountId, TSharedRef<FFriend>>> FriendsLists;
	EOS_HFriends FriendsHandle = nullptr;
	EOS_HUI UIHandle = nullptr;

	FEOSEventRegistrationPtr OnFriendsUpdate;
};

/* UE::Online */ }
