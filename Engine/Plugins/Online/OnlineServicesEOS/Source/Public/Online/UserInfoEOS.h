// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/UserInfoCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_userinfo_types.h"

namespace UE::Online {

class FOnlineServicesEpicCommon;
	
class FUserInfoEOS : public FUserInfoCommon
{
public:
	using Super = FUserInfoCommon;

	ONLINESERVICESEOS_API FUserInfoEOS(FOnlineServicesEpicCommon& InServices);
	virtual ~FUserInfoEOS() = default;

	// IOnlineComponent
	ONLINESERVICESEOS_API virtual void Initialize() override;

	// IUserInfo
	ONLINESERVICESEOS_API virtual TOnlineAsyncOpHandle<FQueryUserInfo> QueryUserInfo(FQueryUserInfo::Params&& Params) override;
	ONLINESERVICESEOS_API virtual TOnlineResult<FGetUserInfo> GetUserInfo(FGetUserInfo::Params&& Params) override;

protected:
	EOS_HUserInfo UserInfoHandle = nullptr;
};

/* UE::Online */ }
